[CoreRedirects]
+PropertyRedirects=(OldName="RigPhysicsSolverSettings.bAutomaticallyAddPhysicsComponents",NewName="bAutomaticallyAddPhysicsBodyComponents")
