{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ControlRigPhysics",
	"Description": "Support for physics simulation in control rig",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"CanContainVerse": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"EnabledByDefault": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "PhysicsControl",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "ControlRigPhysics",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ControlRigPhysicsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
