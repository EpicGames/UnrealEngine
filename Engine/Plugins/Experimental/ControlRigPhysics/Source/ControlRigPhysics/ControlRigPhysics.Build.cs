// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ControlRigPhysics : ModuleRules
{
	public ControlRigPhysics(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"PhysicsControl"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ControlRig",
				"Core",
				"CoreUObject",
				"Engine",
				"RigVM"
			}
		);

		SetupModulePhysicsSupport(Target);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RigVMDeveloper",
					"ControlRigDeveloper",
					"Slate",
					"SlateCore",
					"EditorStyle",
				}
			);
		}
	}
}
