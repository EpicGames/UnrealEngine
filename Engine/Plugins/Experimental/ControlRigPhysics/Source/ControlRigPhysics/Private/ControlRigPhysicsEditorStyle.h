// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Brushes/SlateRoundedBoxBrush.h"

class FControlRigPhysicsEditorStyle final
	: public FSlateStyleSet
{
public:
	FControlRigPhysicsEditorStyle()
		: FSlateStyleSet("ControlRigPhysicsEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		const FString RigPluginContentDir = FPaths::EnginePluginsDir() / TEXT("Experimental/ControlRigPhysics/Content");
		SetContentRoot(RigPluginContentDir);

		// Component Icons
		{
			Set("ControlRigPhysics.Component.Solver", new IMAGE_BRUSH_SVG("Slate/Solver_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodyMultipleDefault", new IMAGE_BRUSH_SVG("Slate/BodyMultipleDefault_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodyMultipleKinematic", new IMAGE_BRUSH_SVG("Slate/BodyMultipleKinematic_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodyMultipleSimulated", new IMAGE_BRUSH_SVG("Slate/BodyMultipleSimulated_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodySingleDefault", new IMAGE_BRUSH_SVG("Slate/BodySingleDefault_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodySingleKinematic", new IMAGE_BRUSH_SVG("Slate/BodySingleKinematic_16", Icon16x16));
			Set("ControlRigPhysics.Component.BodySingleSimulated", new IMAGE_BRUSH_SVG("Slate/BodySingleSimulated_16", Icon16x16));
		}
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FControlRigPhysicsEditorStyle& Get()
	{
		static FControlRigPhysicsEditorStyle Inst;
		return Inst;
	}
	
	~FControlRigPhysicsEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
