// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigPhysicsModule.h"
#include "RigPhysicsObjectVersion.h"

#include "UObject/DevObjectVersion.h"

#if WITH_EDITOR
#include "ControlRigPhysicsEditorStyle.h"
#endif

#define LOCTEXT_NAMESPACE "FControlRigPhysicsModule"

DEFINE_LOG_CATEGORY(LogControlRigPhysics);


// Unique version id for serializing - must not collide with FRigDynamicsObjectVersion::GUID or
// FPhysicsControlObjectVersion::GUID.
const FGuid FRigPhysicsObjectVersion::GUID(0xB7F3D285, 0x4A1C5E08, 0x2D6B9F71, 0xC50E8A14);

// Register custom version with Core. 
FDevVersionRegistration GRegisterRigPhysicsObjectVersion(
	FRigPhysicsObjectVersion::GUID, FRigPhysicsObjectVersion::LatestVersion, TEXT("Dev-RigPhysics"));

//======================================================================================================================
void FControlRigPhysicsModule::StartupModule()
{
#if WITH_EDITOR
	// register the editor style
	FControlRigPhysicsEditorStyle::Get();
#endif
}

//======================================================================================================================
void FControlRigPhysicsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FControlRigPhysicsModule, ControlRigPhysics)

