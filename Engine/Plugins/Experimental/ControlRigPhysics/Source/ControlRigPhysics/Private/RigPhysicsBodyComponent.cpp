// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsBodyComponent.h"
#include "PhysicsControlObjectVersion.h"
#include "RigPhysicsObjectVersion.h"
#include "RigPhysicsSolverComponent.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "ControlRigPhysicsModule.h"
#include "BoxTypes.h"

#if WITH_EDITOR
#include "ControlRigPhysicsEditorStyle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsBodyComponent)

//======================================================================================================================
void FRigPhysicsBodyComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPhysicsControlObjectVersion::GUID);
	Ar.UsingCustomVersion(FRigPhysicsObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	Ar << BodySolverSettings;
	Ar << Dynamics;
	Ar << Collision;
	Ar << BodyData;
	Ar << KinematicTarget;
	Ar << NoCollisionBodies;
}

//======================================================================================================================
void FRigPhysicsBodyComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Ar << BodySolverSettings;
	Ar << Dynamics;
	Ar << Collision;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::ControlRigSeparateOutJointFromBody)
	{
		FRigPhysicsJointData Joint;
		Ar << Joint;
	}
	Ar << BodyData;
	Ar << KinematicTarget;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::RemoveUseSkeletalAnimation)
	{
		EPhysicsControlKinematicTargetSpace KinematicTargetSpace;
		Ar << KinematicTargetSpace;
	}

	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) <
		FPhysicsControlObjectVersion::ControlRigRemoveCurrentDataFromPhysicsComponent)
	{
		FPhysicsControlModifierData CurrentBodyData;
		Ar << CurrentBodyData;
		// Previously people needed to set things in the current data for initial properties, as
		// this would override. So do the override here.
		BodyData = CurrentBodyData;
	}

	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >=
		FPhysicsControlObjectVersion::ControlRigSupportNoCollisionBodies)
	{
		Ar << NoCollisionBodies;
	}
}

//======================================================================================================================
#if WITH_EDITOR
const FSlateIcon& FRigPhysicsBodyComponent::GetIconForUI() const
{
	static const FSlateIcon BodyMultipleDefaultIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodyMultipleDefault");
	static const FSlateIcon BodyMultipleKinematicIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodyMultipleKinematic");
	static const FSlateIcon BodyMultipleSimulatedIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodyMultipleSimulated");
	static const FSlateIcon BodySingleDefaultIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodySingleDefault");
	static const FSlateIcon BodySingleKinematicIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodySingleKinematic");
	static const FSlateIcon BodySingleSimulatedIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.BodySingleSimulated");

	const int32 NumShapes = Collision.NumShapes();;
	if(NumShapes == 0)
	{
		return BodySingleDefaultIcon;
	}
	
	if(BodyData.MovementType == EPhysicsMovementType::Simulated)
	{
		return NumShapes <= 1 ? BodySingleSimulatedIcon : BodyMultipleSimulatedIcon;
	}

	// todo: do we want another set of icons for the static movement type?
	return NumShapes <= 1 ? BodySingleKinematicIcon : BodyMultipleKinematicIcon;
}
#endif

//======================================================================================================================
void FRigPhysicsBodyComponent::OnAddedToHierarchy(URigHierarchy* InHierarchy, URigHierarchyController* InController)
{
	if (!IsProcedural())
	{
		AutoCalculateCollision(InHierarchy);
	}
}

//======================================================================================================================
void FRigPhysicsBodyComponent::AutoCalculateCollision(
	URigHierarchy* InHierarchy, float MinAspectRatio, float InMinSize)
{
	// Start clean
	Collision = FRigPhysicsCollision();
	const float MinSize = FMath::Max(InMinSize, UE_KINDA_SMALL_NUMBER);

	TArray<FVector> Points{ FVector::ZeroVector };
	FVector MidPoint = FVector::ZeroVector;
	TArray<FRigElementKey> ChildKeys = InHierarchy->GetChildren(GetElementKey());
	for (FRigElementKey ChildKey : ChildKeys)
	{
		FVector ChildPosition = InHierarchy->GetLocalTransform(ChildKey, true).GetTranslation();
		if (ChildPosition.SquaredLength() > UE_SMALL_NUMBER)
		{
			Points.Add(ChildPosition);
			MidPoint += ChildPosition;
		}
	}
	MidPoint /= Points.Num();

	if (Points.Num() == 1)
	{
		// If there's only one point, then there are no significant children. Make a shape that
		// duplicates the relationship with our parent, if there is one
		FTransform TM = InHierarchy->GetLocalTransform(GetElementKey(), true);
		if (TM.GetTranslation().SquaredLength() < UE_SMALL_NUMBER)
		{
			// We have no children, and are co-located with our parent. Make a single, minimally
			// sized, sphere.
			Collision.Spheres.Add(FRigPhysicsCollisionSphere(TM, 0.5f * MinSize));
		}
		else
		{
			TM.SetTranslation(TM.GetTranslation() * 0.5);
			// We're going to make a capsule, and they are defined as extending along the Z axis
			TM.SetRotation(FQuat::FindBetweenVectors(FVector(0, 0, 1), TM.GetTranslation()));

			float Length = TM.GetTranslation().Length() * 2.0f;
			const float Radius = 0.5f * FMath::Max(MinSize, Length * MinAspectRatio);
			Length -= 2.0f * Radius;
			Length = FMath::Max(0.0f, Length);
			if (Length > UE_KINDA_SMALL_NUMBER)
			{
				Collision.Capsules.Add(FRigPhysicsCollisionCapsule(TM, Radius, Length));
			}
			else
			{
				Collision.Spheres.Add(FRigPhysicsCollisionSphere(TM, Radius));
			}

		}
	}
	else if (Points.Num() == 2)
	{
		// There's just one child - easier to do this by hand than the more complex eigenvector
		// based calculation, and better than using an OBB which may not be aligned with the two points.
		FTransform TM(MidPoint);
		// We're going to make a capsule, and they are defined as extending along the Z axis
		TM.SetRotation(FQuat::FindBetweenVectors(FVector(0, 0, 1), Points[1]));

		float Length = TM.GetTranslation().Length() * 2.0f;
		const float Radius = 0.5f * FMath::Max(MinSize, Length * MinAspectRatio);
		Length -= 2.0f * Radius;
		if (Length > UE_KINDA_SMALL_NUMBER)
		{
			Collision.Capsules.Add(FRigPhysicsCollisionCapsule(TM, Radius, Length));
		}
		else
		{
			Collision.Spheres.Add(FRigPhysicsCollisionSphere(TM, Radius));
		}
	}
	else
	{
		// We could calculate the ideal orientation of a box by calculating the eigenvectors of the
		// covariance matrix that represents all the joint positions relative to the centroid.
		// However, for now just use a box orientated with the current joint - it is simpler, and
		// also avoids generating "messy" orientations.
		UE::Geometry::FAxisAlignedBox3d Box;
		Box.Contain(Points);

		const FTransform TM(Box.Center());
		FVector Extents = Box.Extents() * 2.0; // The returned extents are half the extents!
		const double MaxExtent = Extents.GetAbsMax();
		Extents.X = FMath::Max(MinSize, FMath::Max(Extents.X, MaxExtent * MinAspectRatio));
		Extents.Y = FMath::Max(MinSize, FMath::Max(Extents.Y, MaxExtent * MinAspectRatio));
		Extents.Z = FMath::Max(MinSize, FMath::Max(Extents.Z, MaxExtent * MinAspectRatio));
		Collision.Boxes.Add(FRigPhysicsCollisionBox(TM, Extents));
	}

	InHierarchy->Notify(ERigHierarchyNotification::ComponentContentChanged, this);
}

//======================================================================================================================
void FRigPhysicsBodyComponent::OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey)
{
	FRigBaseComponent::OnRigHierarchyKeyChanged(InOldKey, InNewKey);

	BodySolverSettings.OnRigHierarchyKeyChanged(InOldKey, InNewKey);
}
