// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsBodyExecution.h"
#include "ControlRigPhysicsModule.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsBodyExecution)

//======================================================================================================================
inline FRigPhysicsBodyComponent* GetBody(URigHierarchy* Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsBodyComponent>(Hierarchy->FindComponent(Key));
}

//======================================================================================================================
FRigUnit_AddPhysicsBody_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AddPhysicsBody can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		PhysicsBodyComponentKey = Controller->AddComponent(
			FRigPhysicsBodyComponent::StaticStruct(), FRigPhysicsBodyComponent::GetDefaultName(), Owner);
		if (PhysicsBodyComponentKey.IsValid())
		{
			if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
			{
				Component->BodySolverSettings = Solver;
				Component->Dynamics = Dynamics;
				Component->BodyData = BodyData;
				Component->Collision = Collision;
				if (Collision.IsEmpty())
				{
					Component->AutoCalculateCollision(ExecuteContext.Hierarchy);
				}
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsBodyExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = Cast<FRigPhysicsBodyComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsBodyComponentKey)) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchyAutoCalculateCollision_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AutoCalculateCollision can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->AutoCalculateCollision(ExecuteContext.Hierarchy, MinAspectRatio, MinSize);
	}
}

//======================================================================================================================
FRigUnit_HierarchySetBodySolverSettings_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SetBodySolverSettings can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodySolverSettings = BodySolverSettings;
	}
}


//======================================================================================================================
FRigUnit_HierarchySetDynamics_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SetDynamics can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->Dynamics = Dynamics;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetCollision_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SetCollision can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->Collision = Collision;
	}
}

//======================================================================================================================
FRigUnit_HierarchyDisableCollisionBetween_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("DisableCollisionBetween can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component1 = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey1))
	{
		Component1->NoCollisionBodies.Add(PhysicsBodyComponentKey2);
	}
}


//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodySourceBone_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodySolverSettings.SourceBone = SourceBone;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyTargetBone_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SetPhysicsBodyTargetBone can only be used during Setup"));
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodySolverSettings.TargetBone = TargetBone;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyKinematicTarget_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->KinematicTarget = KinematicTarget;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyKinematicTargetSpace_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.KinematicTargetSpace = KinematicTargetSpace;
	}
}

//======================================================================================================================
// TODO Note that the sparse data does not get displayed correctly, so this is largely
// unusable - the flags that enable / disable all end up getting reset if the user attempts to
// change them.
FRigUnit_HierarchySetPhysicsBodySparseData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.UpdateFromSparseData(Data);
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyMovementType_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.MovementType = MovementType;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyEnableCCD_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.bEnableCCD = bEnableCCD;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyCollisionType_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.CollisionType = CollisionType;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyIncludeInChecksForReset_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodySolverSettings.bIncludeInChecksForReset = bInclude;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyMaterial_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SetPhysicsBodyMaterial can only be used during Setup"));
		return;
	}

	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->Collision.Material = Material;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyGravityMultiplier_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.GravityMultiplier = GravityMultiplier;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyPhysicsBlendWeight_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.PhysicsBlendWeight = PhysicsBlendWeight;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyUpdateKinematicFromSimulation_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->BodyData.bUpdateKinematicFromSimulation = bUpdateKinematicFromSimulation;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsBodyDamping_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->Dynamics.LinearDamping = LinearDamping;
		Component->Dynamics.AngularDamping = AngularDamping;
	}
}

//======================================================================================================================
FRigUnit_HierarchyAddPhysicsBodyForce_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->ForceAndTorques.Add(FPhysicsControlNamedForceAndTorqueData(Name, ForceAndTorque));
	}
}

//======================================================================================================================
FRigUnit_HierarchyRemovePhysicsBodyForce_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Component->ForceAndTorques.RemoveAll([Name](const FPhysicsControlNamedForceAndTorqueData& Data) 
			{
				return Data.Name == Name;
			});
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetPhysicsBodyTransform_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Transform = Component->Transform;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetPhysicsBodyCoMTransform_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		CoMTransform = Component->CoMTransform;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetPhysicsBodyLinearVelocity_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		LinearVelocity = Component->LinearVelocity;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetPhysicsBodyAngularVelocity_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		AngularVelocity = Component->AngularVelocity;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetPhysicsBodyPointVelocity_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigPhysicsBodyComponent* Component = GetBody(ExecuteContext.Hierarchy, PhysicsBodyComponentKey))
	{
		Velocity = Component->LinearVelocity;
		FVector AngularVelocity = Component->AngularVelocity;
		FVector Location;
		switch (Space)
		{
			// For control rig, World = Component space
			case EPhysicsControlSpace::World:
			case EPhysicsControlSpace::Component:
				Location = Position;
				break;
			case EPhysicsControlSpace::Body:
				Location = Component->Transform.TransformPositionNoScale(Position);
				break;
			default:
				Location = Position;
				break;
		}

		FVector Delta = Location - Component->CoMTransform.GetLocation();
		Velocity += FVector::CrossProduct(AngularVelocity, Delta);
	}
}


