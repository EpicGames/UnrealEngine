// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsControlComponent.h"
#include "PhysicsControlObjectVersion.h"

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsControlComponent)
void FRigPhysicsControlComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPhysicsControlObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	Ar << ParentBodyComponentKey;
	Ar << bUseParentBodyAsDefault;
	Ar << ChildBodyComponentKey;
	Ar << ControlData;
	Ar << ControlTarget;
	Ar << ControlMultiplier;
}

//======================================================================================================================
void FRigPhysicsControlComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Ar << ParentBodyComponentKey;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >=
		FPhysicsControlObjectVersion::ControlRigSeparateOutJointFromBody)
	{
		Ar << bUseParentBodyAsDefault;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >=
		FPhysicsControlObjectVersion::ControlRigControlAddChildBodyComponentKey)
	{
		Ar << ChildBodyComponentKey;
	}
	Ar << ControlData;
	Ar << ControlTarget;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >=
		FPhysicsControlObjectVersion::ControlRigControlAddChildBodyComponentKey)
	{
		Ar << ControlMultiplier;
	}
	DirtyFlags = ERigPhysicsControlComponentDirtyFlags::All;
}

//======================================================================================================================
bool FRigPhysicsControlComponent::CanBeAddedTo(
	const FRigElementKey& InElementKey, const URigHierarchy* InHierarchy, FString* OutFailureReason) const
{
	if (InElementKey.Type != ERigElementType::Bone)
	{
		if (OutFailureReason)
		{
			*OutFailureReason = TEXT("Physics control components can only be added to bones.");
		}
		return false;
	}
	return true;
}

//======================================================================================================================
void FRigPhysicsControlComponent::OnRigHierarchyKeyChanged(
	const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey)
{
	FRigBaseComponent::OnRigHierarchyKeyChanged(InOldKey, InNewKey);

	if (InOldKey.IsComponent() && InNewKey.IsComponent())
	{
		if (ParentBodyComponentKey == InOldKey.GetComponent())
		{
			ParentBodyComponentKey = InNewKey.GetComponent();
		}
		if (ChildBodyComponentKey == InOldKey.GetComponent())
		{
			ChildBodyComponentKey = InNewKey.GetComponent();
		}
	}
	DirtyFlags = ERigPhysicsControlComponentDirtyFlags::All;
}
