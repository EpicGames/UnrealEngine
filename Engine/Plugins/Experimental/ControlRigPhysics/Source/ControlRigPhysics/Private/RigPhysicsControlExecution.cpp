// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsControlExecution.h"

#include "PhysicsControlHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsControlExecution)

//======================================================================================================================
inline FRigPhysicsControlComponent* GetControl(URigHierarchy* Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsControlComponent>(Hierarchy->FindComponent(Key));
}

//======================================================================================================================
FRigUnit_AddPhysicsControl_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AddPhysicsControl can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());

		ControlComponentKey = Controller->AddComponent(FRigPhysicsControlComponent::StaticStruct(), 
			FRigPhysicsControlComponent::GetDefaultName(), Owner);
		if (ControlComponentKey.IsValid())
		{
			if (FRigPhysicsControlComponent* Component = Cast<FRigPhysicsControlComponent>(
				ExecuteContext.Hierarchy->FindComponent(ControlComponentKey)))
			{
				Component->ParentBodyComponentKey = ParentBodyComponentKey;
				Component->bUseParentBodyAsDefault = bUseParentBodyAsDefault;
				Component->ChildBodyComponentKey = ChildBodyComponentKey;
				Component->ControlData = ControlData;
				Component->ControlMultiplier = ControlMultiplier;
				Component->ControlTarget = ControlTarget;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsControlExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = Cast<FRigPhysicsControlComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsControlComponentKey)) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchySetControlEnabled_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlData.bEnabled = bEnabled;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlCustomControlPoint_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			if (Component->ControlData.CustomControlPoint != CustomControlPoint ||
				Component->ControlData.bUseCustomControlPoint != bUseCustomControlPoint)
			{
				Component->ControlData.CustomControlPoint = CustomControlPoint;
				Component->ControlData.bUseCustomControlPoint = bUseCustomControlPoint;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			if (Component->ControlData != ControlData)
			{
				Component->ControlData = ControlData;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetControlData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (const FRigPhysicsControlComponent* Component =
			GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			ControlData = Component->ControlData;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlLinearStrength_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlData.LinearStrength = Strength;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlLinearDampingRatio_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlData.LinearDampingRatio = DampingRatio;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlTargetVelocityMultipliers_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlData.LinearTargetVelocityMultiplier = LinearTargetVelocityMultiplier;
			Component->ControlData.AngularTargetVelocityMultiplier = AngularTargetVelocityMultiplier;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlAngularStrength_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlData.AngularStrength = Strength;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlAngularDampingRatio_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			if (Component->ControlData.AngularDampingRatio != DampingRatio)
			{
				Component->ControlData.AngularDampingRatio = DampingRatio;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlMultiplier_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			if (Component->ControlMultiplier != ControlMultiplier)
			{
				Component->ControlMultiplier = ControlMultiplier;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlDataAndMultiplier_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			if (Component->ControlData != ControlData)
			{
				Component->ControlData = ControlData;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
			if (Component->ControlMultiplier != ControlMultiplier)
			{
				Component->ControlMultiplier = ControlMultiplier;
				Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Data;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetControlTarget_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlTarget = ControlTarget;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Target;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchyUpdateControlTarget_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (ExecuteContext.Hierarchy)
	{
		if (FRigPhysicsControlComponent* Component = GetControl(ExecuteContext.Hierarchy, PhysicsControlComponentKey))
		{
			Component->ControlTarget.TargetVelocity = UE::PhysicsControl::CalculateLinearVelocity(
				TargetPosition, Component->ControlTarget.TargetPosition, DeltaTime);
			Component->ControlTarget.TargetAngularVelocity = UE::PhysicsControl::CalculateAngularVelocity(
				TargetOrientation.Quaternion(), Component->ControlTarget.TargetOrientation.Quaternion(), DeltaTime);
			Component->ControlTarget.TargetPosition = TargetPosition;
			Component->ControlTarget.TargetOrientation = TargetOrientation;
			Component->DirtyFlags |= ERigPhysicsControlComponentDirtyFlags::Target;
		}
	}
}

