// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsData.h"
#include "PhysicsControlObjectVersion.h"
#include "RigPhysicsObjectVersion.h"
#include "RigPhysicsLegacyConversion.h"
#include "PhysicsControlHelpers.h"
#include "Chaos/ChaosConstraintSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsData)

//======================================================================================================================
FRigPhysicsVisualizationSettings1::FRigPhysicsVisualizationSettings1(const FRigPhysicsVisualizationSettings& Other)
{
	LineThickness = Other.LineThickness;
	ShapeSize = Other.ShapeSize;
	ShapeDetail = Other.ShapeDetail;
	bShowBodies = Other.bEnableVisualization && Other.bShowBodies;
	bShowCentreOfMass = Other.bEnableVisualization && Other.bShowCentreOfMass;
	bShowJoints = Other.bEnableVisualization && Other.bShowJoints;
	bShowControls = Other.bEnableVisualization && Other.bShowControls;
	bShowWorldObjects = Other.bEnableVisualization && Other.bShowWorldObjects;
	bShowWorldOverlapBox = Other.bEnableVisualization && Other.bShowWorldOverlapBox;
	bShowActiveContacts = Other.bEnableVisualization && Other.bShowActiveContacts;
	bShowInactiveContacts = Other.bEnableVisualization && Other.bShowInactiveContacts;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionShape& Data)
{
	Ar << Data.RestOffset;
	Ar << Data.Name;
	Ar << Data.bContributeToMass;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionBox& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigShapeData)
	{
		Ar << static_cast<FRigPhysicsCollisionShape&>(Data);
	}
	Ar << Data.TM;
	Ar << Data.Extents;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionSphere& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigShapeData)
	{
		Ar << static_cast<FRigPhysicsCollisionShape&>(Data);
	}
	Ar << Data.TM;
	Ar << Data.Radius;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionCapsule& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigShapeData)
	{
		Ar << static_cast<FRigPhysicsCollisionShape&>(Data);
	}
	Ar << Data.TM;
	Ar << Data.Radius;
	Ar << Data.Length;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionConvex& Data)
{
	Ar << static_cast<FRigPhysicsCollisionShape&>(Data);
	Ar << Data.TM;
	Ar << Data.VertexData;
	return Ar;
}

//======================================================================================================================
FArchive& ArchiveConstraintBaseParams(FArchive& Ar, FConstraintBaseParams& Data)
{
	Ar << Data.Stiffness;
	Ar << Data.Damping;
	Ar << Data.Restitution;
	Ar << Data.ContactDistance;
	bool bSoftConstraint = Data.bSoftConstraint;
	Ar << bSoftConstraint;
	Data.bSoftConstraint = bSoftConstraint;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FLinearConstraint& Data)
{
	ArchiveConstraintBaseParams(Ar, Data);
	Ar << Data.Limit;
	Ar << Data.XMotion;
	Ar << Data.YMotion;
	Ar << Data.ZMotion;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FConeConstraint& Data)
{
	ArchiveConstraintBaseParams(Ar, Data);
	Ar << Data.Swing1LimitDegrees;
	Ar << Data.Swing2LimitDegrees;
	Ar << Data.Swing1Motion;
	Ar << Data.Swing2Motion;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FTwistConstraint& Data)
{
	ArchiveConstraintBaseParams(Ar, Data);
	Ar << Data.TwistLimitDegrees;
	Ar << Data.TwistMotion;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FConstraintDrive& Data)
{
	Ar << Data.Stiffness;
	Ar << Data.Damping;
	Ar << Data.MaxForce;
	bool bEnablePositionDrive = Data.bEnablePositionDrive;
	bool bEnableVelocityDrive = Data.bEnableVelocityDrive;
	Ar << bEnablePositionDrive;
	Ar << bEnableVelocityDrive;
	Data.bEnablePositionDrive = bEnablePositionDrive;
	Data.bEnableVelocityDrive = bEnableVelocityDrive;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FLinearDriveConstraint& Data)
{
	Ar << Data.PositionTarget;
	Ar << Data.VelocityTarget;
	Ar << Data.XDrive;
	Ar << Data.YDrive;
	Ar << Data.ZDrive;
	Ar << Data.bAccelerationMode;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FAngularDriveConstraint& Data)
{
	Ar << Data.TwistDrive;
	Ar << Data.SwingDrive;
	Ar << Data.SlerpDrive;
	Ar << Data.OrientationTarget;
	Ar << Data.AngularVelocityTarget;
	Ar << Data.AngularDriveMode;
	Ar << Data.bAccelerationMode;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsJointData& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigJointEnabled)
	{
		Ar << Data.bEnabled;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::ControlRigSeparateOutJointFromBody)
	{
		bool bEnable;
		FRigComponentKey ParentBody;
		Ar << bEnable;
		Ar << ParentBody;
	}
	Ar << Data.bAutoCalculateParentOffset;
	Ar << Data.bAutoCalculateChildOffset;
	Ar << Data.ExtraParentOffset;
	Ar << Data.ExtraChildOffset;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::ControlRigSupportFullConstraintData)
	{
		float LinearLimit = 0.0f;
		FVector AngularLimit = FVector(-1.0);
		Ar << LinearLimit;
		Ar << AngularLimit;
		if (Ar.IsLoading())
		{
			Data.LinearConstraint.bSoftConstraint = false;
			Data.ConeConstraint.bSoftConstraint = false;
			Data.TwistConstraint.bSoftConstraint = false;

			Data.LinearConstraint.Limit = LinearLimit;
			Data.TwistConstraint.TwistLimitDegrees = AngularLimit.X;
			Data.ConeConstraint.Swing1LimitDegrees = AngularLimit.Y;
			Data.ConeConstraint.Swing2LimitDegrees = AngularLimit.Z;

			Data.LinearConstraint.XMotion = ELinearConstraintMotion::LCM_Limited;
			Data.LinearConstraint.YMotion = ELinearConstraintMotion::LCM_Limited;
			Data.LinearConstraint.ZMotion = ELinearConstraintMotion::LCM_Limited;

			if (Data.LinearConstraint.Limit == 0)
			{
				Data.LinearConstraint.XMotion = ELinearConstraintMotion::LCM_Locked;
			}
			else if (Data.LinearConstraint.Limit < 0)
			{
				Data.LinearConstraint.XMotion = ELinearConstraintMotion::LCM_Free;
			}

			Data.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Limited;
			Data.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Limited;
			Data.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Limited;

			if (Data.TwistConstraint.TwistLimitDegrees == 0)
			{
				Data.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Locked;
			}
			else if (Data.TwistConstraint.TwistLimitDegrees < 0)
			{
				Data.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Free;
			}

			if (Data.ConeConstraint.Swing1LimitDegrees == 0)
			{
				Data.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Locked;
			}
			else if (Data.ConeConstraint.Swing1LimitDegrees < 0)
			{
				Data.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Free;
			}

			if (Data.ConeConstraint.Swing2LimitDegrees == 0)
			{
				Data.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Locked;
			}
			else if (Data.ConeConstraint.Swing2LimitDegrees < 0)
			{
				Data.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Free;
			}

		}
	}
	else
	{
		Ar << Data.LinearConstraint;
		Ar << Data.ConeConstraint;
		Ar << Data.TwistConstraint;
	}
	Ar << Data.bDisableCollision;
	Ar << Data.LinearProjectionAmount;
	Ar << Data.AngularProjectionAmount;
	Ar << Data.ParentInverseMassScale;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigIncludeDriveInJoint)
	{
		if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::ControlRigSeparateOutJointFromBody)
		{
			FRigPhysicsDriveData Drive;
			Ar << Drive;
		}
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationSpaceSettings& Data)
{
	Ar << Data.SpaceMovementAmount;
	Ar << Data.VelocityScaleZ;
	Ar << Data.bClampLinearVelocity;
	Ar << Data.MaxLinearVelocity;
	Ar << Data.bClampAngularVelocity;
	Ar << Data.MaxAngularVelocity;
	Ar << Data.bClampLinearAcceleration;
	Ar << Data.MaxLinearAcceleration;
	Ar << Data.bClampAngularAcceleration;
	Ar << Data.MaxAngularAcceleration;
	Ar << Data.LinearAccelerationThresholdForTeleport;
	Ar << Data.AngularAccelerationThresholdForTeleport;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectTeleportFromDistanceChange)
	{
		Ar << Data.PositionChangeThresholdForTeleport;
		Ar << Data.OrientationChangeThresholdForTeleport;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectTeleportFromDistanceChange)
	{
		Ar << Data.LinearDragMultiplier;
		Ar << Data.AngularDragMultiplier;
	}
	Ar << Data.ExternalLinearDrag;
	Ar << Data.ExternalLinearVelocity;
	Ar << Data.ExternalAngularVelocity;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectExternalVelocityTurbulence)
	{
		Ar << Data.ExternalTurbulenceVelocity;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsInertialForceSettings& Data)
{
	Ar << Data.Amount;
	Ar << Data.LinearEulerAmount;
	Ar << Data.AngularEulerAmount;
	Ar << Data.CentrifugalAmount;
	Ar << Data.CoriolisAmount;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationDragSettings& Data)
{
	Ar << Data.LinearDragMultiplier;
	Ar << Data.AngularDragMultiplier;
	Ar << Data.ExternalLinearDrag;
	Ar << Data.ExternalLinearVelocity;
	Ar << Data.ExternalAngularVelocity;
	Ar << Data.ExternalTurbulenceVelocity;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationSpaceMotion& Data)
{
	Ar << Data.VerticalMotionScale;
	Ar << Data.bClampLinearVelocity;
	Ar << Data.MaxLinearVelocity;
	Ar << Data.bClampAngularVelocity;
	Ar << Data.MaxAngularVelocity;
	Ar << Data.bClampLinearAcceleration;
	Ar << Data.MaxLinearAcceleration;
	Ar << Data.bClampAngularAcceleration;
	Ar << Data.MaxAngularAcceleration;
	Ar << Data.InertialForces;
	Ar << Data.Drag;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsTeleportDetectionSettings& Data)
{
	Ar << Data.bFromPositionChange;
	Ar << Data.PositionChangeThreshold;
	Ar << Data.bFromOrientationChange;
	Ar << Data.OrientationChangeThreshold;
	Ar << Data.bFromLinearAcceleration;
	Ar << Data.LinearAccelerationThreshold;
	Ar << Data.bFromAngularAcceleration;
	Ar << Data.AngularAccelerationThreshold;
	return Ar;
}

//======================================================================================================================
// Decomposes a pre-SimulationSpaceRegrouping FRigPhysicsSimulationSpaceSettings record into the
// new SpaceMotion + TeleportDetection layout. Reads bytes from the archive in the same order as
// FRigPhysicsSimulationSpaceSettings::operator<< above.
void TranslateLegacyPhysicsSimulationSpaceSettings(
	FArchive& Ar,
	FRigPhysicsSimulationSpaceMotion& OutMotion,
	FRigPhysicsTeleportDetectionSettings& OutTeleport)
{
	float SpaceMovementAmount = 1.0f;
	float VelocityScaleZ = 1.0f;
	bool bClampLinVel = false, bClampAngVel = false, bClampLinAcc = false, bClampAngAcc = false;
	float MaxLinVel = 10000.0f, MaxAngVel = 10000.0f, MaxLinAcc = 10000.0f, MaxAngAcc = 10000.0f;
	float LinAccThresh = 100000.0f, AngAccThresh = 100000.0f;
	float PosThresh = 100.0f, OriThresh = 30.0f;
	float LinDrag = 1.0f, AngDrag = 1.0f;
	FVector ExtLinDrag = FVector::ZeroVector;
	FVector ExtLinVel = FVector::ZeroVector;
	FVector ExtAngVel = FVector::ZeroVector;
	FVector ExtTurbulence = FVector::ZeroVector;

	Ar << SpaceMovementAmount;
	Ar << VelocityScaleZ;
	Ar << bClampLinVel; Ar << MaxLinVel;
	Ar << bClampAngVel; Ar << MaxAngVel;
	Ar << bClampLinAcc; Ar << MaxLinAcc;
	Ar << bClampAngAcc; Ar << MaxAngAcc;
	Ar << LinAccThresh;
	Ar << AngAccThresh;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectTeleportFromDistanceChange)
	{
		Ar << PosThresh;
		Ar << OriThresh;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectTeleportFromDistanceChange)
	{
		Ar << LinDrag;
		Ar << AngDrag;
	}
	Ar << ExtLinDrag;
	Ar << ExtLinVel;
	Ar << ExtAngVel;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDetectExternalVelocityTurbulence)
	{
		Ar << ExtTurbulence;
	}

	OutMotion.VerticalMotionScale          = VelocityScaleZ;
	OutMotion.bClampLinearVelocity         = bClampLinVel;
	OutMotion.MaxLinearVelocity            = MaxLinVel;
	OutMotion.bClampAngularVelocity        = bClampAngVel;
	OutMotion.MaxAngularVelocity           = MaxAngVel;
	OutMotion.bClampLinearAcceleration     = bClampLinAcc;
	OutMotion.MaxLinearAcceleration        = MaxLinAcc;
	OutMotion.bClampAngularAcceleration    = bClampAngAcc;
	OutMotion.MaxAngularAcceleration       = MaxAngAcc;
	OutMotion.InertialForces.Amount        = SpaceMovementAmount;
	// Per-term gains weren't present in the legacy struct; default 1.0 reproduces the Chaos *Alpha
	// constructor defaults that the legacy code relied on.
	OutMotion.InertialForces.LinearEulerAmount  = 1.0f;
	OutMotion.InertialForces.AngularEulerAmount = 1.0f;
	OutMotion.InertialForces.CentrifugalAmount  = 1.0f;
	OutMotion.InertialForces.CoriolisAmount     = 1.0f;
	OutMotion.Drag.LinearDragMultiplier        = LinDrag;
	OutMotion.Drag.AngularDragMultiplier       = AngDrag;
	OutMotion.Drag.ExternalLinearDrag          = ExtLinDrag;
	OutMotion.Drag.ExternalLinearVelocity      = ExtLinVel;
	OutMotion.Drag.ExternalAngularVelocity     = ExtAngVel;
	OutMotion.Drag.ExternalTurbulenceVelocity  = ExtTurbulence;

	// Legacy "0 = disabled" mapping. Per user policy, when the threshold loaded as 0 we replace it
	// with the constructor default so toggling the gate back on later reveals a sensible value.
	OutTeleport.bFromLinearAcceleration       = LinAccThresh > 0.0f;
	OutTeleport.LinearAccelerationThreshold   = LinAccThresh > 0.0f ? LinAccThresh : 100000.0f;
	OutTeleport.bFromAngularAcceleration      = AngAccThresh > 0.0f;
	OutTeleport.AngularAccelerationThreshold  = AngAccThresh > 0.0f ? AngAccThresh : 100000.0f;
	OutTeleport.bFromPositionChange           = PosThresh    > 0.0f;
	OutTeleport.PositionChangeThreshold       = PosThresh    > 0.0f ? PosThresh    : 100.0f;
	OutTeleport.bFromOrientationChange        = OriThresh    > 0.0f;
	OutTeleport.OrientationChangeThreshold    = OriThresh    > 0.0f ? OriThresh    : 30.0f;
}

//======================================================================================================================
// In-memory equivalent of TranslateLegacyPhysicsSimulationSpaceSettings (no archive read). Used by
// the deprecated rig units to convert their input pin into the new component members.
void ConvertLegacyPhysicsSimulationSpaceSettings(
	const FRigPhysicsSimulationSpaceSettings& In,
	FRigPhysicsSimulationSpaceMotion& OutMotion,
	FRigPhysicsTeleportDetectionSettings& OutTeleport)
{
	OutMotion.VerticalMotionScale          = In.VelocityScaleZ;
	OutMotion.bClampLinearVelocity         = In.bClampLinearVelocity;
	OutMotion.MaxLinearVelocity            = In.MaxLinearVelocity;
	OutMotion.bClampAngularVelocity        = In.bClampAngularVelocity;
	OutMotion.MaxAngularVelocity           = In.MaxAngularVelocity;
	OutMotion.bClampLinearAcceleration     = In.bClampLinearAcceleration;
	OutMotion.MaxLinearAcceleration        = In.MaxLinearAcceleration;
	OutMotion.bClampAngularAcceleration    = In.bClampAngularAcceleration;
	OutMotion.MaxAngularAcceleration       = In.MaxAngularAcceleration;
	OutMotion.InertialForces.Amount        = In.SpaceMovementAmount;
	OutMotion.InertialForces.LinearEulerAmount  = 1.0f;
	OutMotion.InertialForces.AngularEulerAmount = 1.0f;
	OutMotion.InertialForces.CentrifugalAmount  = 1.0f;
	OutMotion.InertialForces.CoriolisAmount     = 1.0f;
	OutMotion.Drag.LinearDragMultiplier        = In.LinearDragMultiplier;
	OutMotion.Drag.AngularDragMultiplier       = In.AngularDragMultiplier;
	OutMotion.Drag.ExternalLinearDrag          = In.ExternalLinearDrag;
	OutMotion.Drag.ExternalLinearVelocity      = In.ExternalLinearVelocity;
	OutMotion.Drag.ExternalAngularVelocity     = In.ExternalAngularVelocity;
	OutMotion.Drag.ExternalTurbulenceVelocity  = In.ExternalTurbulenceVelocity;

	OutTeleport.bFromLinearAcceleration       = In.LinearAccelerationThresholdForTeleport > 0.0f;
	OutTeleport.LinearAccelerationThreshold   = In.LinearAccelerationThresholdForTeleport > 0.0f
		? In.LinearAccelerationThresholdForTeleport : 100000.0f;
	OutTeleport.bFromAngularAcceleration      = In.AngularAccelerationThresholdForTeleport > 0.0f;
	OutTeleport.AngularAccelerationThreshold  = In.AngularAccelerationThresholdForTeleport > 0.0f
		? In.AngularAccelerationThresholdForTeleport : 100000.0f;
	OutTeleport.bFromPositionChange           = In.PositionChangeThresholdForTeleport > 0.0f;
	OutTeleport.PositionChangeThreshold       = In.PositionChangeThresholdForTeleport > 0.0f
		? In.PositionChangeThresholdForTeleport : 100.0f;
	OutTeleport.bFromOrientationChange        = In.OrientationChangeThresholdForTeleport > 0.0f;
	OutTeleport.OrientationChangeThreshold    = In.OrientationChangeThresholdForTeleport > 0.0f
		? In.OrientationChangeThresholdForTeleport : 30.0f;
}

//======================================================================================================================
// Inverse of ConvertLegacyPhysicsSimulationSpaceSettings - composes a legacy settings view from the
// new component members so the deprecated Get rig unit can present its output pin. Per-term
// inertial gains are dropped (the legacy struct doesn't hold them); a gate=false on a teleport
// detector translates to threshold=0 (the legacy "disabled" convention).
FRigPhysicsSimulationSpaceSettings BuildLegacyPhysicsSimulationSpaceSettingsView(
	const FRigPhysicsSimulationSpaceMotion& Motion,
	const FRigPhysicsTeleportDetectionSettings& Teleport)
{
	FRigPhysicsSimulationSpaceSettings Out;
	Out.SpaceMovementAmount        = Motion.InertialForces.Amount;
	Out.VelocityScaleZ             = Motion.VerticalMotionScale;
	Out.bClampLinearVelocity       = Motion.bClampLinearVelocity;
	Out.MaxLinearVelocity          = Motion.MaxLinearVelocity;
	Out.bClampAngularVelocity      = Motion.bClampAngularVelocity;
	Out.MaxAngularVelocity         = Motion.MaxAngularVelocity;
	Out.bClampLinearAcceleration   = Motion.bClampLinearAcceleration;
	Out.MaxLinearAcceleration      = Motion.MaxLinearAcceleration;
	Out.bClampAngularAcceleration  = Motion.bClampAngularAcceleration;
	Out.MaxAngularAcceleration     = Motion.MaxAngularAcceleration;
	Out.LinearDragMultiplier       = Motion.Drag.LinearDragMultiplier;
	Out.AngularDragMultiplier      = Motion.Drag.AngularDragMultiplier;
	Out.ExternalLinearDrag         = Motion.Drag.ExternalLinearDrag;
	Out.ExternalLinearVelocity     = Motion.Drag.ExternalLinearVelocity;
	Out.ExternalAngularVelocity    = Motion.Drag.ExternalAngularVelocity;
	Out.ExternalTurbulenceVelocity = Motion.Drag.ExternalTurbulenceVelocity;

	Out.LinearAccelerationThresholdForTeleport  = Teleport.bFromLinearAcceleration  ? Teleport.LinearAccelerationThreshold  : 0.0f;
	Out.AngularAccelerationThresholdForTeleport = Teleport.bFromAngularAcceleration ? Teleport.AngularAccelerationThreshold : 0.0f;
	Out.PositionChangeThresholdForTeleport      = Teleport.bFromPositionChange      ? Teleport.PositionChangeThreshold      : 0.0f;
	Out.OrientationChangeThresholdForTeleport   = Teleport.bFromOrientationChange   ? Teleport.OrientationChangeThreshold   : 0.0f;
	return Out;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsSolverSettings& Data)
{
	Ar << Data.SimulationSpace;
	Ar << Data.CollisionSpace;
	Ar << Data.SpaceBone;
	Ar << Data.Collision;
	Ar << Data.Gravity;
	Ar << Data.PositionIterations;
	Ar << Data.VelocityIterations;
	Ar << Data.ProjectionIterations;
	Ar << Data.MaxNumRollingAverageStepTimes;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigSolverSettingsIncludesCollisionBoundsExpansion)
	{
		Ar << Data.CollisionBoundsExpansion;
		Ar << Data.BoundsVelocityMultiplier;
		Ar << Data.MaxVelocityBoundsExpansion;
	}
	Ar << Data.MaxDepenetrationVelocity;
	Ar << Data.FixedTimeStep;
	Ar << Data.MaxTimeSteps;
	Ar << Data.MaxDeltaTime;
	Ar << Data.bUseLinearJointSolver;
	Ar << Data.bSolveJointPositionsLast;
	Ar << Data.bUseManifolds;

	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigAllowCCD)
	{
		Ar << Data.bAllowCCD;
	}

	Ar << Data.PositionThresholdForReset;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigSpeedThresholdForReset)
	{
		Ar << Data.KinematicSpeedThresholdForReset;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigAccelerationThresholdForReset)
	{
		Ar << Data.KinematicAccelerationThresholdForReset;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) < FPhysicsControlObjectVersion::ControlRigRemoveResetCooldownFrames)
	{
		if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigResetCooldownFrames)
		{
			int ResetCooldownFrames = 0;
			Ar << ResetCooldownFrames;
		}
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigAutomaticallyAddPhysicsBodyComponents)
	{
		Ar << Data.bAutomaticallyAddPhysicsBodyComponents;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigWorldCollision)
	{
		Ar << Data.WorldCollisionType;
		if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) < FRigPhysicsObjectVersion::WorldCollisionExpiryFramesRemoved)
		{
			// Legacy field: expiry now hardcoded to one frame past LastSeen in FWorldObject::GetExpired.
			int32 LegacyWorldCollisionExpiryFrames = 0;
			Ar << LegacyWorldCollisionExpiryFrames;
		}
		Ar << Data.WorldCollisionBoundsExpansion;
	}

	// Before WorldCollisionFilterFlags, the three include-flags didn't exist on disk. The
	// constructor defaults of true (RigPhysicsData.h) reproduce the legacy "always include all
	// categories" behaviour, so no explicit fallback branch is needed.
	if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) >= FRigPhysicsObjectVersion::WorldCollisionFilterFlags)
	{
		Ar << Data.bWorldCollisionIncludePhysics;
		Ar << Data.bWorldCollisionIncludeQuery;
		Ar << Data.bWorldCollisionIncludeProbe;
	}

	if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) >= FRigPhysicsObjectVersion::ResetGatesSplit)
	{
		Ar << Data.bResetFromPosition;
		Ar << Data.bResetFromKinematicSpeed;
		Ar << Data.bResetFromKinematicAcceleration;
	}
	else if (Ar.IsLoading())
	{
		// Pre-ResetGatesSplit: gate inferred from "threshold > 0".
		Data.bResetFromPosition = Data.PositionThresholdForReset > 0.0f;
		Data.bResetFromKinematicSpeed = Data.KinematicSpeedThresholdForReset       > 0.0f;
		Data.bResetFromKinematicAcceleration = Data.KinematicAccelerationThresholdForReset > 0.0f;

		// User policy: substitute constructor default when threshold loaded as 0, so toggling the
		// gate back on shows a sensible value. Defaults match the bumped struct defaults.
		if (Data.PositionThresholdForReset == 0.0f) 
		{
			Data.PositionThresholdForReset = 10000.0f; 
		}
		if (Data.KinematicSpeedThresholdForReset == 0.0f) 
		{ 
			Data.KinematicSpeedThresholdForReset = 5000.0f;  
		}
		if (Data.KinematicAccelerationThresholdForReset == 0.0f)
		{
			Data.KinematicAccelerationThresholdForReset = 100000.0f;
		}
	}

	if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) >= FRigPhysicsObjectVersion::EvaluationIntervalReset)
	{
		Ar << Data.bResetFromEvaluationInterval;
		Ar << Data.EvaluationIntervalThresholdForReset;
	}

	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsDriveData& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigSupportFullDriveConstraintData)
	{
		Ar << Data.LinearDriveConstraint;
		Ar << Data.AngularDriveConstraint;
	}
	else
	{
		bool bEnable = false;
		float LinearStrength = 0.0f;
		float LinearDampingRatio = 1.0f;
		float LinearExtraDamping = 0.0f;
		float MaxForce = 0.0;
		float AngularStrength = 0.0f;
		float AngularDampingRatio = 1.0f;
		float AngularExtraDamping = 0.0f;
		float MaxTorque = 0.0f;
		
		Ar << bEnable;
		Ar << LinearStrength;
		Ar << LinearDampingRatio;
		Ar << LinearExtraDamping;
		Ar << MaxForce;
		Ar << AngularStrength;
		Ar << AngularDampingRatio;
		Ar << AngularExtraDamping;
		Ar << MaxTorque;

		if (Ar.IsLoading())
		{
			// Convert to the constraint drive params
			float AngularSpring;
			float AngularDamping;

			float LinearSpring;
			float LinearDamping;

			UE::PhysicsControl::ConvertStrengthToSpringParams(
				AngularSpring, AngularDamping,
				AngularStrength, AngularDampingRatio, AngularExtraDamping);
			UE::PhysicsControl::ConvertStrengthToSpringParams(
				LinearSpring, LinearDamping,
				LinearStrength, LinearDampingRatio, LinearExtraDamping);

			// Unfortunately, the physics engine will apply scalings to these values, so we need to
			// counter that.
			LinearSpring /= Chaos::ConstraintSettings::LinearDriveStiffnessScale();
			LinearDamping /= Chaos::ConstraintSettings::LinearDriveDampingScale();
			AngularSpring /= Chaos::ConstraintSettings::AngularDriveStiffnessScale();
			AngularDamping /= Chaos::ConstraintSettings::AngularDriveDampingScale();

			Data.LinearDriveConstraint.XDrive.Stiffness = LinearSpring;
			Data.LinearDriveConstraint.YDrive.Stiffness = LinearSpring;
			Data.LinearDriveConstraint.ZDrive.Stiffness = LinearSpring;

			Data.LinearDriveConstraint.XDrive.Damping = LinearDamping;
			Data.LinearDriveConstraint.YDrive.Damping = LinearDamping;
			Data.LinearDriveConstraint.ZDrive.Damping = LinearDamping;

			Data.LinearDriveConstraint.XDrive.MaxForce = MaxForce;
			Data.LinearDriveConstraint.YDrive.MaxForce = MaxForce;
			Data.LinearDriveConstraint.ZDrive.MaxForce = MaxForce;

			Data.AngularDriveConstraint.SlerpDrive.Stiffness = AngularSpring;
			Data.AngularDriveConstraint.SwingDrive.Stiffness = AngularSpring;
			Data.AngularDriveConstraint.TwistDrive.Stiffness = AngularSpring;

			Data.AngularDriveConstraint.SlerpDrive.Damping = AngularDamping;
			Data.AngularDriveConstraint.SwingDrive.Damping = AngularDamping;
			Data.AngularDriveConstraint.TwistDrive.Damping = AngularDamping;

			Data.AngularDriveConstraint.SlerpDrive.MaxForce = MaxTorque;
			Data.AngularDriveConstraint.SwingDrive.MaxForce = MaxTorque;
			Data.AngularDriveConstraint.TwistDrive.MaxForce = MaxTorque;

			Data.LinearDriveConstraint.XDrive.bEnablePositionDrive = bEnable;
			Data.LinearDriveConstraint.YDrive.bEnablePositionDrive = bEnable;
			Data.LinearDriveConstraint.ZDrive.bEnablePositionDrive = bEnable;

			Data.LinearDriveConstraint.XDrive.bEnableVelocityDrive = bEnable;
			Data.LinearDriveConstraint.YDrive.bEnableVelocityDrive = bEnable;
			Data.LinearDriveConstraint.ZDrive.bEnableVelocityDrive = bEnable;

			Data.AngularDriveConstraint.SlerpDrive.bEnablePositionDrive = bEnable;
			Data.AngularDriveConstraint.SwingDrive.bEnablePositionDrive = bEnable;
			Data.AngularDriveConstraint.TwistDrive.bEnablePositionDrive = bEnable;

			Data.AngularDriveConstraint.SlerpDrive.bEnableVelocityDrive = bEnable;
			Data.AngularDriveConstraint.SwingDrive.bEnableVelocityDrive = bEnable;
			Data.AngularDriveConstraint.TwistDrive.bEnableVelocityDrive = bEnable;
		}
	}
	Ar << Data.SkeletalAnimationVelocityMultiplier;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigDriveRelativeToAnimation)
	{
		Ar << Data.bUseSkeletalAnimation;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsBodySolverSettings& Data)
{
	Ar << Data.PhysicsSolverComponentKey;
	Ar << Data.TargetBone;
	Ar << Data.SourceBone;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigUseAutomaticSolver)
	{
		Ar << Data.bUseAutomaticSolver;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigBodyIncludeInChecksForReset)
	{
		Ar << Data.bIncludeInChecksForReset;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsCollision& Data)
{
	Ar << Data.Boxes;
	Ar << Data.Spheres;
	Ar << Data.Capsules;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigCollisionHasMaterial)
	{
		Ar << Data.Material;
	}
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigConvexCollision)
	{
		Ar << Data.Convexes;
	}
	return Ar;
}

//======================================================================================================================
void FRigPhysicsBodySolverSettings::OnRigHierarchyKeyChanged(
	const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey)
{
	if(InOldKey.IsComponent() && InNewKey.IsComponent())
	{
		if(PhysicsSolverComponentKey == InOldKey.GetComponent())
		{
			PhysicsSolverComponentKey = InNewKey.GetComponent();
		}
	}
	if(InOldKey.IsElement() && InNewKey.IsElement())
	{
		if(SourceBone == InOldKey.GetElement())
		{
			SourceBone = InNewKey.GetElement();
		}
		if(TargetBone == InOldKey.GetElement())
		{
			TargetBone = InNewKey.GetElement();
		}
	}
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsDynamics& Data)
{
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigBodyDynamicsHasDensity)
	{
		Ar << Data.Density;
	}
	Ar << Data.MassOverride;
	Ar << Data.bOverrideCentreOfMass;
	Ar << Data.CentreOfMassOverride;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigCentreOfMassNudge)
	{
		Ar << Data.CentreOfMassNudge;
	}
	Ar << Data.bOverrideMomentsOfInertia;
	Ar << Data.MomentsOfInertiaOverride;
	if (Ar.CustomVer(FPhysicsControlObjectVersion::GUID) >= FPhysicsControlObjectVersion::ControlRigSupportBodyDamping)
	{
		Ar << Data.LinearDamping;
		Ar << Data.AngularDamping;
	}
	if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) >= FRigPhysicsObjectVersion::BodyDampingScaleByInverseMass)
	{
		Ar << Data.bScaleDampingByInverseMass;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigPhysicsMaterial& Data)
{
	Ar << Data.Friction;
	Ar << Data.Restitution;
	Ar << Data.FrictionCombineMode;
	Ar << Data.RestitutionCombineMode;
	return Ar;
}
