// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsExecution.h"
#include "RigPhysicsJointComponent.h"
#include "RigPhysicsControlComponent.h"
#include "RigPhysicsSolver.h"
#include "ControlRigPhysicsModule.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRig.h"

#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsExecution)

//======================================================================================================================
FRigUnit_AddPhysicsComponents_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AddPhysicsComponents can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		// Add the body
		PhysicsBodyComponentKey = Controller->AddComponent(
			FRigPhysicsBodyComponent::StaticStruct(), FRigPhysicsBodyComponent::GetDefaultName(), Owner);
		if (PhysicsBodyComponentKey.IsValid())
		{
			if (FRigPhysicsBodyComponent* Component = Cast<FRigPhysicsBodyComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsBodyComponentKey)))
			{
				Component->BodySolverSettings = Solver;
				Component->Dynamics = Dynamics;
				Component->BodyData = BodyData;
				Component->Collision = Collision;
				if (Collision.IsEmpty())
				{
					Component->AutoCalculateCollision(ExecuteContext.Hierarchy);
				}
			}

			if (bAddJoint)
			{
				PhysicsJointComponentKey = Controller->AddComponent(
					FRigPhysicsJointComponent::StaticStruct(), FRigPhysicsJointComponent::GetDefaultName(), Owner);
				if (PhysicsJointComponentKey.IsValid())
				{
					if (FRigPhysicsJointComponent* Component = Cast<FRigPhysicsJointComponent>(
						ExecuteContext.Hierarchy->FindComponent(PhysicsJointComponentKey)))
					{
						// Leave parent as blank to automatically find it
						Component->ChildBodyComponentKey = PhysicsBodyComponentKey;
						Component->JointData = JointData;
						Component->DriveData = DriveData;
						Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::All;
					}
				}
			}

			if (bAddSimSpaceControl)
			{
				SimSpaceControlComponentKey = Controller->AddComponent(FRigPhysicsControlComponent::StaticStruct(),
					TEXT("SimSpaceControl"), Owner);
				if (SimSpaceControlComponentKey.IsValid())
				{
					if (FRigPhysicsControlComponent* Component = Cast<FRigPhysicsControlComponent>(
						ExecuteContext.Hierarchy->FindComponent(SimSpaceControlComponentKey)))
					{
						Component->bUseParentBodyAsDefault = false;
						Component->ChildBodyComponentKey = PhysicsBodyComponentKey;
						Component->ControlData = SimSpaceControlData;
					}
				}
			}

			if (bAddParentSpaceControl)
			{
				ParentSpaceControlComponentKey = Controller->AddComponent(FRigPhysicsControlComponent::StaticStruct(),
					TEXT("ParentSpaceControl"), Owner);
				if (ParentSpaceControlComponentKey.IsValid())
				{
					if (FRigPhysicsControlComponent* Component = Cast<FRigPhysicsControlComponent>(
						ExecuteContext.Hierarchy->FindComponent(ParentSpaceControlComponentKey)))
					{
						Component->bUseParentBodyAsDefault = true;
						Component->ChildBodyComponentKey = PhysicsBodyComponentKey;
						Component->ControlData = ParentSpaceControlData;
					}
				}
			}
		}
	}
}

//======================================================================================================================
static void SetCommonProperties(FRigPhysicsCollisionShape& Shape, const FKShapeElem& ShapeElem)
{
	Shape.RestOffset = ShapeElem.RestOffset;
	Shape.Name = ShapeElem.GetName();
	Shape.bContributeToMass = ShapeElem.GetContributeToMass();
}

//======================================================================================================================
static void SetUpJointData(
	FRigPhysicsJointData&               JointData, 
	const FConstraintInstance&          ConstraintInstance, 
	const FConstraintProfileProperties& ConstraintProfileProperties)
{
	JointData.bAutoCalculateChildOffset = false;
	JointData.bAutoCalculateParentOffset = false;

	JointData.ExtraChildOffset = ConstraintInstance.GetRefFrame(EConstraintFrame::Frame1);
	JointData.ExtraParentOffset = ConstraintInstance.GetRefFrame(EConstraintFrame::Frame2);

	JointData.ExtraParentOffset.SetRotation(JointData.ExtraParentOffset.GetRotation() *
		ConstraintInstance.AngularRotationOffset.Quaternion());

	JointData.LinearConstraint = ConstraintProfileProperties.LinearLimit;
	JointData.ConeConstraint = ConstraintProfileProperties.ConeLimit;
	JointData.TwistConstraint = ConstraintProfileProperties.TwistLimit;

	JointData.LinearProjectionAmount = ConstraintProfileProperties.ProjectionLinearAlpha;
	JointData.AngularProjectionAmount = ConstraintProfileProperties.ProjectionAngularAlpha;
}

//======================================================================================================================
static FRigComponentKey FindPhysicsBodyComponentWithSolver(
	const URigHierarchy* Hierarchy, const FRigElementKey& ElementKey, const FRigComponentKey& SolverKey)
{
	// Note that our owning element might have multiple bodies, and we can't be sure of the name. So
	// look for one that has the same solver as us - there should only be one of those.
	const TArray<FRigComponentKey> CandidateBodyComponentKeys =
		Hierarchy->GetComponentKeys(ElementKey);
	if (const FRigComponentKey* CC = CandidateBodyComponentKeys.FindByPredicate(
		[SolverKey, &Hierarchy](const FRigComponentKey& Other) {
			if (const FRigPhysicsBodyComponent* PBC = Cast<FRigPhysicsBodyComponent>(
				Hierarchy->FindComponent(Other)))
			{
				if (PBC->BodySolverSettings.bUseAutomaticSolver ||
					PBC->BodySolverSettings.PhysicsSolverComponentKey == SolverKey)
				{
					return true;
				}
			}
			return false;
		}))
	{
		return *CC;
	}

	return FRigComponentKey();

}

//======================================================================================================================
FRigUnit_HierarchyInstantiateFromPhysicsAsset_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("InstantiateFromPhysicsAsset can only be used during Setup"));
		return;
	}

	// The construction event can be re-entered in preview without re-instantiating the unit, so
	// wipe any output arrays from prior runs before we start appending fresh keys.
	PhysicsBodyComponentKeys.Reset();
	PhysicsJointComponentKeys.Reset();
	SimSpaceControlComponentKeys.Reset();
	ParentSpaceControlComponentKeys.Reset();

	if (!PhysicsAsset)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("InstantiateFromPhysicsAsset needs a valid physics asset"));
		return;
	}

	URigHierarchyController * Controller = ExecuteContext.Hierarchy->GetController();
	if (!Controller)
	{
		return;
	}

	FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());

	// When we create physics joints we need to be able to refer back to the bodies that have been created
	TMap<FName, FRigComponentKey> BoneToBodyComponentKeyMap;

	for (TObjectPtr<USkeletalBodySetup> SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (!SkeletalBodySetup)
		{
			continue;
		}

		// If this flag is set then it means the body under consideration is a helper body - a
		// kinematic one that is not in BonesToUse, but is needed to support the bodies we should be creating.
		bool bNeedBodyAsHelper = false;

		// Check we can use this based on the bone name
		const FName BoneName = SkeletalBodySetup->BoneName;
		const FRigElementKey OwnerElementKey(BoneName, ERigElementType::Bone);
		if (!ExecuteContext.Hierarchy->Find(OwnerElementKey))
		{
			continue;
		}
		if (!BonesToUse.IsEmpty() && BonesToUse.Find(OwnerElementKey) == INDEX_NONE)
		{
			if (bMakeHelperBodies)
			{
				// Whilst this bone isn't in the list of bones to use, it may be connected via a
				// constraint (physics joint) to a bone that is in use. If we find that this is the
				// case, we can make a kinematic body for it (flagged as a helper body) so that
				// we're able to form joints and controls between it and bodies that are in BonesToUse.

				// TODO this nested O(N) search is not ideal
				for (const TObjectPtr<UPhysicsConstraintTemplate>& ConstraintTemplate : PhysicsAsset->ConstraintSetup)
				{
					if (!ConstraintTemplate)
					{
						continue;
					}
					FConstraintInstance& ConstraintInstance = ConstraintTemplate->DefaultInstance;
					const FName ChildBoneName = ConstraintInstance.ConstraintBone1;
					const FName ParentBoneName = ConstraintInstance.ConstraintBone2;

					if (BoneName == ParentBoneName &&
						ExecuteContext.Hierarchy->Find(FRigElementKey(ChildBoneName, ERigElementType::Bone)) &&
						BonesToUse.Find(FRigElementKey(ChildBoneName, ERigElementType::Bone)) != INDEX_NONE)
					{
						bNeedBodyAsHelper = true;
						break;
					}

					// Constraints can also be set up in the other order (though less commonly)
					if (BoneName == ChildBoneName &&
						ExecuteContext.Hierarchy->Find(FRigElementKey(ParentBoneName, ERigElementType::Bone)) &&
						BonesToUse.Find(FRigElementKey(ParentBoneName, ERigElementType::Bone)) != INDEX_NONE)
					{
						bNeedBodyAsHelper = true;
						break;
					}
				}

				if (!bNeedBodyAsHelper)
				{
					continue;
				}
			}
			else
			{
				continue;
			}
		}

		// This is the body component that we will (one of):
		// * Make from scratch
		// * Re-use, but update
		// * Re-use as-is

		// Note that this will return an invalid key if an existing body component can't be found
		FRigComponentKey PhysicsBodyComponentKey = FindPhysicsBodyComponentWithSolver(
			ExecuteContext.Hierarchy, OwnerElementKey, PhysicsSolverComponentKey);

		// Note that the body (PhysicsBodyComponentKey) might exist, but even then it might have
		// been created as a helper. In that case, we don't need to recreate it, but we do need to
		// overwrite its properties.

		// If the key is not valid, we will make the body. But we also need to decide if we will
		// overwrite the collision/dynamics properties, and whether we force it to be kinematic.

		// Need helper      Key Valid       Make body   Overwrite properties   Force kinematic
		// 0                0               1           1                      0
		// 0                1               0           1                      0
		// 1                0               1           1                      1
		// 1                1               0           0                      0

		if (!(PhysicsBodyComponentKey.IsValid() && bNeedBodyAsHelper))
		{
			FRigPhysicsCollision Collision;
			FRigPhysicsDynamics Dynamics;
			FPhysicsControlModifierData BodyData;

			// Collision
			const FKAggregateGeom& AggGeom = SkeletalBodySetup->AggGeom;
			for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
			{
				FTransform TM(BoxElem.Rotation, BoxElem.Center);
				FRigPhysicsCollisionBox Box(TM, FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
				SetCommonProperties(Box, BoxElem);
				Collision.Boxes.Add(Box);
			}
			for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
			{
				FTransform TM(SphereElem.Center);
				FRigPhysicsCollisionSphere Sphere(TM, SphereElem.Radius);
				SetCommonProperties(Sphere, SphereElem);
				Collision.Spheres.Add(Sphere);
			}
			for (const FKSphylElem& SphylElem : AggGeom.SphylElems)
			{
				FTransform TM(SphylElem.Rotation, SphylElem.Center);
				FRigPhysicsCollisionCapsule Capsule(TM, SphylElem.Radius, SphylElem.Length);
				SetCommonProperties(Capsule, SphylElem);
				Collision.Capsules.Add(Capsule);
			}
			for (const FKConvexElem& ConvexElem : AggGeom.ConvexElems)
			{
				FRigPhysicsCollisionConvex Convex(ConvexElem.GetTransform(), ConvexElem.VertexData);
				SetCommonProperties(Convex, ConvexElem);
				Collision.Convexes.Add(Convex);
			}

			const UPhysicalMaterial* Material = SkeletalBodySetup->GetPhysMaterial();
			Material = Material ? Material : GetDefault<UPhysicalMaterial>();
			if (Material)
			{
				Collision.Material.Friction = FMath::Max(Material->Friction, Material->StaticFriction);
				Collision.Material.Restitution = Material->Restitution;
				Collision.Material.FrictionCombineMode = (ERigPhysicsCombineMode)
					Material->FrictionCombineMode.GetValue();
				Collision.Material.RestitutionCombineMode = (ERigPhysicsCombineMode)
					Material->RestitutionCombineMode.GetValue();
			}

			// Dynamics
			Dynamics.MassOverride = SkeletalBodySetup->CalculateMass();
			Dynamics.LinearDamping = SkeletalBodySetup->DefaultInstance.LinearDamping;
			Dynamics.AngularDamping = SkeletalBodySetup->DefaultInstance.AngularDamping;
			Dynamics.CentreOfMassNudge = SkeletalBodySetup->DefaultInstance.COMNudge;

			// Body Data
			BodyData.MovementType =
				SkeletalBodySetup->PhysicsType == EPhysicsType::PhysType_Simulated ?
				EPhysicsMovementType::Simulated : EPhysicsMovementType::Kinematic;
			BodyData.CollisionType =
				SkeletalBodySetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Enabled ?
				ECollisionEnabled::PhysicsOnly : ECollisionEnabled::NoCollision;
			BodyData.bEnableCCD = SkeletalBodySetup->DefaultInstance.bUseCCD;

			if (bNeedBodyAsHelper)
			{
				BodyData.MovementType = EPhysicsMovementType::Kinematic;
				BodyData.CollisionType = ECollisionEnabled::NoCollision;
			}

			// Add the component if necessary
			if (!PhysicsBodyComponentKey.IsValid())
			{
				PhysicsBodyComponentKey = Controller->AddComponent(
					FRigPhysicsBodyComponent::StaticStruct(), TEXT("PhysicsBody"), OwnerElementKey);
			}

			// Update the component's properties
			if (FRigPhysicsBodyComponent* Component = Cast<FRigPhysicsBodyComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsBodyComponentKey)))
			{
				Component->BodySolverSettings.PhysicsSolverComponentKey = PhysicsSolverComponentKey;
				Component->BodySolverSettings.bUseAutomaticSolver = bUseAutomaticSolver;
				Component->Dynamics = Dynamics;
				Component->BodyData = BodyData;
				Component->Collision = Collision;

				BoneToBodyComponentKeyMap.Add(BoneName, PhysicsBodyComponentKey);
				if (!bNeedBodyAsHelper)
				{
					PhysicsBodyComponentKeys.Add(PhysicsBodyComponentKey);
				}
			}
		} // Creating or updating the body
		else
		{
			BoneToBodyComponentKeyMap.Add(BoneName, PhysicsBodyComponentKey);
		}

		// Sim Space control
		if (bAddSimSpaceControl && !bNeedBodyAsHelper)
		{
			FRigComponentKey SimSpaceControlComponentKey = Controller->AddComponent(
				FRigPhysicsControlComponent::StaticStruct(),
				TEXT("SimSpaceControl"), PhysicsBodyComponentKey.ElementKey);
			if (FRigPhysicsControlComponent* Component = Cast<FRigPhysicsControlComponent>(
				ExecuteContext.Hierarchy->FindComponent(SimSpaceControlComponentKey)))
			{
				Component->bUseParentBodyAsDefault = false;
				Component->ChildBodyComponentKey = PhysicsBodyComponentKey;
				Component->ControlData = SimSpaceControlData;

				SimSpaceControlComponentKeys.Add(SimSpaceControlComponentKey);
			}
		}

		// Parent space control
		if (bAddParentSpaceControl && !bNeedBodyAsHelper)
		{
			// Note that we create a parent-space control for every body - even the root-most body.
			// This shouldn't be a problem because we'll set bUseParentBodyAsDefault, and when the
			// control is instantiated, this will be used to distinguish between a genuine sim-space
			// control, and a parent-space control where there is no parent body.
			FRigComponentKey ParentSpaceControlComponentKey = Controller->AddComponent(
				FRigPhysicsControlComponent::StaticStruct(),
				TEXT("ParentSpaceControl"), PhysicsBodyComponentKey.ElementKey);
			if (FRigPhysicsControlComponent* Component = Cast<FRigPhysicsControlComponent>(
				ExecuteContext.Hierarchy->FindComponent(ParentSpaceControlComponentKey)))
			{
				Component->bUseParentBodyAsDefault = true;
				Component->ChildBodyComponentKey = PhysicsBodyComponentKey;
				Component->ControlData = ParentSpaceControlData;

				ParentSpaceControlComponentKeys.Add(ParentSpaceControlComponentKey);
			}
		}
	} // Loop over SkeletalBodySetup

	// Now bodies are created we can handle no-collision
	//
	// TODO Note that this is a loop over the whole collision table in the physics asset -
	// even if we are just pulling out a small number of bones from it. It could potentially be
	// quite expensive, especially as once we have a pair of bones, we need to iterate further to
	// decide whether to keep the pair (especially as we don't want to make duplicates).
	const TMap<FRigidBodyIndexPair, bool>& CollisionDisableTable = PhysicsAsset->CollisionDisableTable;
	for (const TPair<FRigidBodyIndexPair, bool>& PairPair : CollisionDisableTable)
	{
		if (!PairPair.Value)
		{
			const FRigidBodyIndexPair& Pair = PairPair.Key;
			int32 I1 = Pair.Indices[0];
			int32 I2 = Pair.Indices[1];
			if (I1 >= 0 && I1 < PhysicsAsset->SkeletalBodySetups.Num() && 
				I2 >= 0 && I2 < PhysicsAsset->SkeletalBodySetups.Num())
			{
				if (PhysicsAsset->SkeletalBodySetups[I1] && PhysicsAsset->SkeletalBodySetups[I2])
				{
					const FName BoneName1 = PhysicsAsset->SkeletalBodySetups[I1]->BoneName;
					const FName BoneName2 = PhysicsAsset->SkeletalBodySetups[I2]->BoneName;
					bool bInclude1 = (BoneToBodyComponentKeyMap.Find(BoneName1) != nullptr);
					bool bInclude2 = (BoneToBodyComponentKeyMap.Find(BoneName2) != nullptr);

					if (bInclude1 || bInclude2)
					{
						// If the user instantiates from the same physics asset multiple times, for
						// different limbs, and bodies from one limb are set to not collide with the
						// other limbs, then during the first limb processing, the other limb bodies
						// won't exist yet. We don't try to register the no-collision no - we wait
						// until the second (etc) limbs are created.
						//
						// TODO We could skip a lot of these checks/loops if we're creating
						// from the whole physics asset - i.e. it might be worth having an
						// alternative path for that.
						FRigComponentKey BodyComponentKey1 = FindPhysicsBodyComponentWithSolver(
							ExecuteContext.Hierarchy, 
							FRigElementKey(BoneName1, ERigElementType::Bone), 
							PhysicsSolverComponentKey);
						if (BodyComponentKey1.IsValid())
						{
							FRigComponentKey BodyComponentKey2 = FindPhysicsBodyComponentWithSolver(
								ExecuteContext.Hierarchy,
								FRigElementKey(BoneName2, ERigElementType::Bone),
								PhysicsSolverComponentKey);
							if (BodyComponentKey2.IsValid())
							{
								FRigPhysicsBodyComponent* Component1 = Cast<FRigPhysicsBodyComponent>(
									ExecuteContext.Hierarchy->FindComponent(BodyComponentKey1));
								FRigPhysicsBodyComponent* Component2 = Cast<FRigPhysicsBodyComponent>(
									ExecuteContext.Hierarchy->FindComponent(BodyComponentKey2));
								// It probably doesn't really matter, but add the no-collision to
								// the component we're actually working with
								if (bInclude1 && Component1)
								{
									Component1->NoCollisionBodies.AddUnique(BodyComponentKey2);
								}
								else if (Component2)
								{
									Component2->NoCollisionBodies.AddUnique(BodyComponentKey1);
								}
							}
						}
					}
				}
			}
		}
	}

	// Physics Joints
	if (bEnableJoints)
	{
		for (const TObjectPtr<UPhysicsConstraintTemplate>& ConstraintTemplate : PhysicsAsset->ConstraintSetup)
		{
			if (!ConstraintTemplate)
			{
				continue;
			}

			// Note that physics assets are normally set up as child/parent for 1/2
			// However, users can create their own constraints, and some assets will be the other way round.
			FConstraintInstance& ConstraintInstance = ConstraintTemplate->DefaultInstance;
			const FName ChildBoneName = ConstraintInstance.ConstraintBone1;
			const FName ParentBoneName = ConstraintInstance.ConstraintBone2;

			const FRigComponentKey* ChildBodyComponentKey = BoneToBodyComponentKeyMap.Find(ChildBoneName);
			const FRigComponentKey* ParentBodyComponentKey = BoneToBodyComponentKeyMap.Find(ParentBoneName);

			if (!ChildBodyComponentKey || !ParentBodyComponentKey)
			{
				if (BonesToUse.IsEmpty())
				{
					// If BonesToUse is not empty, it is hard to say whether this is an
					// error/warning condition. However, if the physics asset is OK and BonesToUse
					// is empty, then we shouldn't get here.
					UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(
						TEXT("InstantiateFromPhysicsAsset - unable to make physics joint between %s and %s"),
						*ChildBoneName.ToString(), *ParentBoneName.ToString());
				}
				continue;
			}

			const FConstraintProfileProperties& ConstraintProfileProperties = 
				ConstraintTemplate->GetConstraintProfilePropertiesOrDefault(ConstraintProfileName);

			FRigPhysicsJointData JointData;
			SetUpJointData(JointData, ConstraintInstance, ConstraintProfileProperties);
			FRigPhysicsDriveData DriveData;
			if (bEnableDrives)
			{
				DriveData.LinearDriveConstraint = ConstraintProfileProperties.LinearDrive;
				DriveData.AngularDriveConstraint = ConstraintProfileProperties.AngularDrive;
				// RBAN doesn't use skeletal animation, and by default we want to replicate that.
				DriveData.bUseSkeletalAnimation = false;
			}

			// Add the component
			FRigComponentKey PhysicsJointComponentKey = Controller->AddComponent(
				FRigPhysicsJointComponent::StaticStruct(), TEXT("PhysicsJoint"), ChildBodyComponentKey->ElementKey);
			if (PhysicsJointComponentKey.IsValid())
			{
				if (FRigPhysicsJointComponent* Component = Cast<FRigPhysicsJointComponent>(
					ExecuteContext.Hierarchy->FindComponent(PhysicsJointComponentKey)))
				{
					Component->JointData = JointData;
					Component->DriveData = DriveData;
					Component->ParentBodyComponentKey = *ParentBodyComponentKey;
					Component->ChildBodyComponentKey = *ChildBodyComponentKey;
					Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::All;

					PhysicsJointComponentKeys.Add(PhysicsJointComponentKey);
				}
			}
		} // Loop over constraint templates
	}
}

//======================================================================================================================
FRigUnit_HierarchyImportCollisionFromPhysicsAsset_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("InstantiateFromPhysicsAsset can only be used during Setup"));
		return;
	}

	// The construction event can be re-entered in preview without re-instantiating the unit, so
	// wipe any output arrays from prior runs before we start appending fresh keys.
	BoneKeys.Reset();
	PhysicsBodyComponentKeys.Reset();

	if (!PhysicsAsset)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("InstantiateFromPhysicsAsset needs a valid physics asset"));
		return;
	}

	URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController();
	if (!Controller)
	{
		return;
	}

	FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());

	for (TObjectPtr<USkeletalBodySetup> SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (!SkeletalBodySetup)
		{
			continue;
		}

		// Check we can use this based on the bone name
		FName PhysicsBoneName = SkeletalBodySetup->BoneName;

		if (!BonesToUse.IsEmpty() && BonesToUse.Find(PhysicsBoneName) == INDEX_NONE)
		{
			continue;
		}

		FName BoneName(NameSpace.ToString() + PhysicsBoneName.ToString());
		FRigElementKey BoneKey = Controller->AddBone(BoneName, Owner, FTransform(), true, ERigBoneType::Imported);

		if (BoneKey.IsValid())
		{
			// Collision
			FRigPhysicsCollision Collision;
			const FKAggregateGeom& AggGeom = SkeletalBodySetup->AggGeom;
			for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
			{
				FTransform TM(BoxElem.Rotation, BoxElem.Center);
				FRigPhysicsCollisionBox Box(TM, FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
				SetCommonProperties(Box, BoxElem);
				Collision.Boxes.Add(Box);
			}
			for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
			{
				FTransform TM(SphereElem.Center);
				FRigPhysicsCollisionSphere Sphere(TM, SphereElem.Radius);
				SetCommonProperties(Sphere, SphereElem);
				Collision.Spheres.Add(Sphere);
			}
			for (const FKSphylElem& SphylElem : AggGeom.SphylElems)
			{
				FTransform TM(SphylElem.Rotation, SphylElem.Center);
				FRigPhysicsCollisionCapsule Capsule(TM, SphylElem.Radius, SphylElem.Length);
				SetCommonProperties(Capsule, SphylElem);
				Collision.Capsules.Add(Capsule);
			}
			for (const FKConvexElem& ConvexElem : AggGeom.ConvexElems)
			{
				FRigPhysicsCollisionConvex Convex(ConvexElem.GetTransform(), ConvexElem.VertexData);
				SetCommonProperties(Convex, ConvexElem);
				Collision.Convexes.Add(Convex);
			}

			const UPhysicalMaterial* Material = SkeletalBodySetup->GetPhysMaterial();
			Material = Material ? Material : GetDefault<UPhysicalMaterial>();
			if (Material)
			{
				Collision.Material.Friction = FMath::Max(Material->Friction, Material->StaticFriction);
				Collision.Material.Restitution = Material->Restitution;
				Collision.Material.FrictionCombineMode = (ERigPhysicsCombineMode)
					Material->FrictionCombineMode.GetValue();
				Collision.Material.RestitutionCombineMode = (ERigPhysicsCombineMode)
					Material->RestitutionCombineMode.GetValue();
			}

			// Dynamics
			FRigPhysicsDynamics Dynamics;
			Dynamics.MassOverride = SkeletalBodySetup->CalculateMass();
			Dynamics.LinearDamping = SkeletalBodySetup->DefaultInstance.LinearDamping;
			Dynamics.AngularDamping = SkeletalBodySetup->DefaultInstance.AngularDamping;

			// Body Data
			FPhysicsControlModifierData BodyData;
			BodyData.MovementType = EPhysicsMovementType::Kinematic; // Always kinematic
			BodyData.CollisionType =
				SkeletalBodySetup->CollisionReponse == EBodyCollisionResponse::BodyCollision_Enabled ?
				ECollisionEnabled::PhysicsOnly : ECollisionEnabled::NoCollision;
			BodyData.bEnableCCD = SkeletalBodySetup->DefaultInstance.bUseCCD;

			// Add the component
			FRigComponentKey PhysicsBodyComponentKey = Controller->AddComponent(
				FRigPhysicsBodyComponent::StaticStruct(), TEXT("PhysicsBody"), BoneKey);
			if (FRigPhysicsBodyComponent* Component = Cast<FRigPhysicsBodyComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsBodyComponentKey)))
			{
				Component->BodySolverSettings.PhysicsSolverComponentKey = PhysicsSolverComponentKey;
				Component->BodySolverSettings.bUseAutomaticSolver = bUseAutomaticSolver;
				Component->Dynamics = Dynamics;
				Component->BodyData = BodyData;
				Component->Collision = Collision;

				PhysicsBodyComponentKeys.Add(PhysicsBodyComponentKey);
				BoneKeys.Add(BoneKey);
			}
			else
			{
				Controller->RemoveElement(BoneKey);
			}
		}
	}
}
