// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsBodyComponent.h"
#include "RigPhysicsJointComponent.h"
#include "RigPhysicsControlComponent.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyCache.h"

//======================================================================================================================
inline FRigPhysicsBodyComponent* GetPhysicsBody(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsBodyComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigPhysicsBodyComponent* GetPhysicsBody(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsBodyComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component.
inline const FRigPhysicsBodyComponent* GetPhysicsBody(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigPhysicsBodyComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
// Mutable variant of the cached lookup. FCachedRigComponent::GetComponent returns a const pointer
// regardless of hierarchy constness, so the const_cast is safe here because the caller holds a
// non-const URigHierarchy& and the underlying component lives in that hierarchy.
inline FRigPhysicsBodyComponent* GetPhysicsBody(URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return const_cast<FRigPhysicsBodyComponent*>(
		Cast<FRigPhysicsBodyComponent>(CachedComponent.GetComponent(&Hierarchy)));
}

//======================================================================================================================
inline FRigPhysicsJointComponent* GetPhysicsJoint(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsJointComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigPhysicsJointComponent* GetPhysicsJoint(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsJointComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component.
inline const FRigPhysicsJointComponent* GetPhysicsJoint(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigPhysicsJointComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
// Mutable variant of the cached lookup. See the body equivalent for the const_cast rationale.
inline FRigPhysicsJointComponent* GetPhysicsJoint(URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return const_cast<FRigPhysicsJointComponent*>(
		Cast<FRigPhysicsJointComponent>(CachedComponent.GetComponent(&Hierarchy)));
}

//======================================================================================================================
inline FRigPhysicsControlComponent* GetPhysicsControl(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsControlComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigPhysicsControlComponent* GetPhysicsControl(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsControlComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component.
inline const FRigPhysicsControlComponent* GetPhysicsControl(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigPhysicsControlComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
// Mutable variant of the cached lookup. See the body equivalent for the const_cast rationale.
inline FRigPhysicsControlComponent* GetPhysicsControl(URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return const_cast<FRigPhysicsControlComponent*>(
		Cast<FRigPhysicsControlComponent>(CachedComponent.GetComponent(&Hierarchy)));
}

//======================================================================================================================
// Returns the element index of the cached component's owning element, refreshing the cache first.
// Caller is responsible for ensuring the cached component has a seeded key.
inline int32 GetElementIndex(const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	CachedRigComponent.UpdateCache(&Hierarchy);
	const FRigBaseElement* Element = CachedRigComponent.GetElement();
	// Since all components are owned, they must have an element.
	check(Element);
	return Element->GetIndex();
}

//======================================================================================================================
// Fetches the global transform of the cached component's owning element using the cached index,
// avoiding the FRigElementKey TMap lookup that Hierarchy.GetGlobalTransform(Key) performs.
inline FTransform GetGlobalTransform(const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	const int32 Index = GetElementIndex(Hierarchy, CachedRigComponent);
	return Hierarchy.GetGlobalTransform(Index);
}
