// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsJointComponent.h"
#include "PhysicsControlObjectVersion.h"
#include "RigPhysicsSolverComponent.h"
#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "ControlRigPhysicsModule.h"

#if WITH_EDITOR
#include "ControlRigPhysicsEditorStyle.h"
#endif

//======================================================================================================================

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsJointComponent)
void FRigPhysicsJointComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPhysicsControlObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	Ar << ParentBodyComponentKey;
	Ar << ChildBodyComponentKey;
	Ar << JointData;
	Ar << DriveData;
}

//======================================================================================================================
void FRigPhysicsJointComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Ar << ParentBodyComponentKey;
	Ar << ChildBodyComponentKey;
	Ar << JointData;
	Ar << DriveData;
	DirtyFlags = ERigPhysicsJointComponentDirtyFlags::All;
}

//======================================================================================================================
bool FRigPhysicsJointComponent::CanBeAddedTo(
	const FRigElementKey& InElementKey, const URigHierarchy* InHierarchy, FString* OutFailureReason) const
{
	if(InElementKey.Type != ERigElementType::Bone)
	{
		if(OutFailureReason)
		{
			*OutFailureReason = TEXT("Physics joint components can only be added to bones.");
		}
		return false;
	}
	return true;
}


//======================================================================================================================
void FRigPhysicsJointComponent::OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey)
{
	FRigBaseComponent::OnRigHierarchyKeyChanged(InOldKey, InNewKey);

	if (InOldKey.IsComponent() && InNewKey.IsComponent())
	{
		if (ParentBodyComponentKey == InOldKey.GetComponent())
		{
			ParentBodyComponentKey = InNewKey.GetComponent();
		}
		if (ChildBodyComponentKey == InOldKey.GetComponent())
		{
			ChildBodyComponentKey = InNewKey.GetComponent();
		}
	}
	DirtyFlags = ERigPhysicsJointComponentDirtyFlags::All;
}
