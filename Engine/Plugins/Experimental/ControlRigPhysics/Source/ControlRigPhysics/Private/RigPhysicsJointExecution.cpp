// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsJointExecution.h"
#include "ControlRigPhysicsModule.h"

#include "PhysicsControlHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRig.h"

#include "Chaos/ChaosConstraintSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsJointExecution)

//======================================================================================================================
inline FRigPhysicsJointComponent* GetJoint(URigHierarchy* Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigPhysicsJointComponent>(Hierarchy->FindComponent(Key));
}

//======================================================================================================================
FRigUnit_AddPhysicsJoint_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AddPhysicsJoint can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		PhysicsJointComponentKey = Controller->AddComponent(
			FRigPhysicsJointComponent::StaticStruct(), FRigPhysicsJointComponent::GetDefaultName(), Owner);
		if (PhysicsJointComponentKey.IsValid())
		{
			if (FRigPhysicsJointComponent* Component = Cast<FRigPhysicsJointComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsJointComponentKey)))
			{
				Component->ParentBodyComponentKey = ParentBodyComponentKey;
				Component->ChildBodyComponentKey = ChildBodyComponentKey;
				Component->JointData = JointData;
				Component->DriveData = DriveData;
				Component->DirtyFlags = ERigPhysicsJointComponentDirtyFlags::All;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsJointExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = Cast<FRigPhysicsJointComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsJointComponentKey)) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchySetJointData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->JointData = JointData;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Joint;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetJointData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		JointData = Component->JointData;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointLinearLimit_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->JointData.LinearConstraint = LinearLimit.LinearConstraint;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Joint;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointConeLimit_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->JointData.ConeConstraint = ConeLimit.ConeConstraint;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Joint;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointTwistLimit_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->JointData.TwistConstraint = TwistLimit.TwistConstraint;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Joint;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointEnabled_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->JointData.bEnabled = bEnabled;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Joint;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointDriveData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->DriveData = DriveData;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Drive;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetJointDriveData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		DriveData = Component->DriveData;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetJointDriveUseSkeletalAnimation_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsJointComponent* Component = GetJoint(ExecuteContext.Hierarchy, PhysicsJointComponentKey))
	{
		Component->DriveData.bUseSkeletalAnimation = bUseSkeletalAnimation;
		Component->DirtyFlags |= ERigPhysicsJointComponentDirtyFlags::Drive;
	}
}

//======================================================================================================================
FRigUnit_MakeArticulationJointData_Execute()
{
	FVector SoftStiffness;
	FVector SoftDamping;

	UE::PhysicsControl::ConvertStrengthToSpringParams(
		SoftStiffness, SoftDamping,
		SoftStrength, SoftDampingRatio, FVector(0));

	// Unfortunately, the physics engine will apply scalings to these values, so we need to counter that.
	SoftStiffness /= Chaos::ConstraintSettings::SoftAngularStiffnessScale();
	SoftDamping /= Chaos::ConstraintSettings::SoftAngularDampingScale();

	// Twist
	if (AngularLimit.X < 0)
	{
		JointData.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Free;
	}
	else if (AngularLimit.X == 0)
	{
		JointData.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Locked;
	}
	else
	{
		JointData.TwistConstraint.TwistMotion = EAngularConstraintMotion::ACM_Limited;
		JointData.TwistConstraint.TwistLimitDegrees = AngularLimit.X;

		JointData.TwistConstraint.Stiffness = SoftStiffness.X;
		JointData.TwistConstraint.Damping = SoftDamping.X;

		if (SoftStrength.X >= 0)
		{
			JointData.TwistConstraint.bSoftConstraint = true;
		}
		else
		{
			JointData.TwistConstraint.bSoftConstraint = false;
		}
	}

	// Swing 1
	if (AngularLimit.Y < 0)
	{
		JointData.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Free;
	}
	else if (AngularLimit.Y == 0)
	{
		JointData.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Locked;
	}
	else
	{
		JointData.ConeConstraint.Swing1Motion = EAngularConstraintMotion::ACM_Limited;
		JointData.ConeConstraint.Swing1LimitDegrees = AngularLimit.Y;

		JointData.ConeConstraint.Stiffness = SoftStiffness.Y;
		JointData.ConeConstraint.Damping = SoftDamping.Y;

		if (SoftStrength.Y >= 0)
		{
			JointData.ConeConstraint.bSoftConstraint = true;
		}
		else
		{
			JointData.ConeConstraint.bSoftConstraint = false;
		}
	}

	// Swing2
	if (AngularLimit.Z < 0)
	{
		JointData.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Free;
	}
	else if (AngularLimit.Z == 0)
	{
		JointData.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Locked;
	}
	else
	{
		JointData.ConeConstraint.Swing2Motion = EAngularConstraintMotion::ACM_Limited;
		JointData.ConeConstraint.Swing2LimitDegrees = AngularLimit.Z;

		JointData.ConeConstraint.Stiffness = SoftStiffness.Z;
		JointData.ConeConstraint.Damping = SoftDamping.Z;

		if (SoftStrength.Z >= 0)
		{
			JointData.ConeConstraint.bSoftConstraint = true;
		}
		else
		{
			JointData.ConeConstraint.bSoftConstraint = false;
		}
	}

}

//======================================================================================================================
FRigUnit_MakeArticulationDriveData_Execute()
{
	// Convert to the constraint drive params
	float AngularSpring;
	float AngularDamping;

	UE::PhysicsControl::ConvertStrengthToSpringParams(
		AngularSpring, AngularDamping,
		AngularStrength, AngularDampingRatio, AngularExtraDamping);

	// Unfortunately, the physics engine will apply scalings to these values, so we need to counter that.
	AngularSpring /= Chaos::ConstraintSettings::AngularDriveStiffnessScale();
	AngularDamping /= Chaos::ConstraintSettings::AngularDriveDampingScale();

	DriveData.SkeletalAnimationVelocityMultiplier = SkeletalAnimationVelocityMultiplier;

	DriveData.LinearDriveConstraint.SetLinearPositionDrive(false, false, false);
	DriveData.LinearDriveConstraint.SetLinearVelocityDrive(false, false, false);

	if (bEnableAngularDrive)
	{
		DriveData.AngularDriveConstraint.SetOrientationDriveTwistAndSwing(true, true);
		DriveData.AngularDriveConstraint.SetOrientationDriveSLERP(true);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveTwistAndSwing(true, true);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveSLERP(true);

		DriveData.AngularDriveConstraint.SetDriveParams(AngularSpring, AngularDamping, 0.0f);
		DriveData.AngularDriveConstraint.SetAngularDriveMode(AngularDriveMode);
		DriveData.AngularDriveConstraint.SetAccelerationMode(true);
	}
	else
	{
		DriveData.AngularDriveConstraint.SetOrientationDriveTwistAndSwing(false, false);
		DriveData.AngularDriveConstraint.SetOrientationDriveSLERP(false);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveTwistAndSwing(false, false);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveSLERP(false);
	}

}
//======================================================================================================================
FRigUnit_MakeDriveData_Execute()
{
	// Convert to the constraint drive params
	float AngularSpring;
	float AngularDamping;

	float LinearSpring;
	float LinearDamping;

	UE::PhysicsControl::ConvertStrengthToSpringParams(
		AngularSpring, AngularDamping,
		AngularStrength, AngularDampingRatio, AngularExtraDamping);
	UE::PhysicsControl::ConvertStrengthToSpringParams(
		LinearSpring, LinearDamping,
		LinearStrength, LinearDampingRatio, LinearExtraDamping);

	// Unfortunately, the physics engine will apply scalings to these values, so we need to
	// counter that.
	LinearSpring /= Chaos::ConstraintSettings::LinearDriveStiffnessScale();
	LinearDamping /= Chaos::ConstraintSettings::LinearDriveDampingScale();
	AngularSpring /= Chaos::ConstraintSettings::AngularDriveStiffnessScale();
	AngularDamping /= Chaos::ConstraintSettings::AngularDriveDampingScale();

	DriveData.SkeletalAnimationVelocityMultiplier = SkeletalAnimationVelocityMultiplier;

	if (bEnableLinearDrive)
	{
		DriveData.LinearDriveConstraint.SetLinearPositionDrive(true, true, true);
		DriveData.LinearDriveConstraint.SetLinearVelocityDrive(true, true, true);
		DriveData.LinearDriveConstraint.SetDriveParams(FVector(LinearSpring), FVector(LinearDamping), FVector(0));
		DriveData.LinearDriveConstraint.SetAccelerationMode(true);
	}
	else
	{
		DriveData.LinearDriveConstraint.SetLinearPositionDrive(false, false, false);
		DriveData.LinearDriveConstraint.SetLinearVelocityDrive(false, false, false);
	}

	if (bEnableAngularDrive)
	{
		DriveData.AngularDriveConstraint.SetOrientationDriveTwistAndSwing(true, true);
		DriveData.AngularDriveConstraint.SetOrientationDriveSLERP(true);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveTwistAndSwing(true, true);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveSLERP(true);

		DriveData.AngularDriveConstraint.SetDriveParams(AngularSpring, AngularDamping, 0.0f);
		DriveData.AngularDriveConstraint.SetAngularDriveMode(AngularDriveMode);
		DriveData.AngularDriveConstraint.SetAccelerationMode(true);
	}
	else
	{
		DriveData.AngularDriveConstraint.SetOrientationDriveTwistAndSwing(false, false);
		DriveData.AngularDriveConstraint.SetOrientationDriveSLERP(false);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveTwistAndSwing(false, false);
		DriveData.AngularDriveConstraint.SetAngularVelocityDriveSLERP(false);
	}
}
