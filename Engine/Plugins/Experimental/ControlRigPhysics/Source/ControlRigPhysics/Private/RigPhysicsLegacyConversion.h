// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsData.h"

class FArchive;

//======================================================================================================================
// Helpers for migrating from the pre-SimulationSpaceRegrouping FRigPhysicsSimulationSpaceSettings
// shape to the new FRigPhysicsSimulationSpaceMotion + FRigPhysicsTeleportDetectionSettings layout.
// All three live in RigPhysicsData.cpp and are private to the ControlRigPhysics module.
//======================================================================================================================

// Decomposes a pre-SimulationSpaceRegrouping FRigPhysicsSimulationSpaceSettings record into the
// new SpaceMotion + TeleportDetection layout. Reads bytes from the archive in the same order as
// FRigPhysicsSimulationSpaceSettings::operator<<.
void TranslateLegacyPhysicsSimulationSpaceSettings(
	FArchive& Ar,
	FRigPhysicsSimulationSpaceMotion& OutMotion,
	FRigPhysicsTeleportDetectionSettings& OutTeleport);

// In-memory equivalent of TranslateLegacyPhysicsSimulationSpaceSettings (no archive read). Used by
// the deprecated rig units' Execute bodies to convert their input pin into the new component
// members.
void ConvertLegacyPhysicsSimulationSpaceSettings(
	const FRigPhysicsSimulationSpaceSettings& In,
	FRigPhysicsSimulationSpaceMotion& OutMotion,
	FRigPhysicsTeleportDetectionSettings& OutTeleport);

// Inverse of ConvertLegacyPhysicsSimulationSpaceSettings - composes a legacy settings view from
// the new component members so the deprecated Get rig unit can present its output pin. Per-term
// inertial gains are dropped (the legacy struct doesn't hold them); a gate=false on a teleport
// detector translates to threshold=0 (the legacy "disabled" convention).
FRigPhysicsSimulationSpaceSettings BuildLegacyPhysicsSimulationSpaceSettingsView(
	const FRigPhysicsSimulationSpaceMotion& Motion,
	const FRigPhysicsTeleportDetectionSettings& Teleport);
