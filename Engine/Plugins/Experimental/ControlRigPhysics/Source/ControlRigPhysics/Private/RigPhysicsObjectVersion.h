// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API CONTROLRIGPHYSICS_API

//======================================================================================================================
struct FRigPhysicsObjectVersion
{
	enum Type
	{
		FirstVersion,
		SimulationSpaceRegrouping,    // FRigPhysicsSimulationSpaceSettings split into 4 structs
		ResetGatesSplit,              // bResetFromX gates added to FRigPhysicsSolverSettings
		WorldCollisionFilterFlags,    // bWorldCollisionInclude{Physics,Query,Probe} now serialized
		EvaluationIntervalReset,      // bResetFromEvaluationInterval + EvaluationIntervalThresholdForReset
		BodyDampingScaleByInverseMass,// bScaleDampingByInverseMass added to FRigPhysicsDynamics
		WorldCollisionExpiryFramesRemoved, // FRigPhysicsSolverSettings::WorldCollisionExpiryFrames removed

		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FRigPhysicsObjectVersion() {}
};

#undef UE_API
