// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolver.h"
#include "RigPhysicsSolver_Space.inl"

#include "RigPhysicsBodyComponent.h"
#include "RigPhysicsJointComponent.h"
#include "RigPhysicsControlComponent.h"
#include "RigPhysicsSolverComponent.h"
#include "RigPhysicsHelpers.h"

#include "AnimNode_RigidBodyWithControl.h"
#include "PhysicsControlPoseData.h"
#include "PhysicsControlHelpers.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "RigVMCore/RigVMExecuteContext.h"

#include "Engine/World.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsSimulation_Chaos.h"

#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"

#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ShapeInstance.h"
#include "Chaos/Capsule.h"
#include "Chaos/ChaosScene.h"
#include "Chaos/Convex.h"
#include "Chaos/PBDJointConstraintUtilities.h"
#include "Chaos/Evolution/SimulationSpace.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/ChaosDebugNameDefines.h"

#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/BodySetup.h"

#include "Stats/Stats.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVDRuntimeModule.h"
#endif

#include "Logging/LogMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsSolver)

DEFINE_LOG_CATEGORY(LogRigPhysics);

TAutoConsoleVariable<float> CVarControlRigPhysicsFixedTimeStepOverride(
	TEXT("ControlRig.Physics.FixedTimeStepOverride"), -1.0f,
	TEXT("-1.0 disables the override, so the timestep authored in the simulation settings will be used (which may or may not imply a fixed timestep). A value of 0 forces a variable timestep to be used. A +ve value is used to specify a fixed timestep."));

TAutoConsoleVariable<int> CVarControlRigPhysicsMaxTimeStepsOverride(
	TEXT("ControlRig.Physics.MaxTimeStepsOverride"), -1,
	TEXT("-1 disables the override, so the max timesteps authored in the simulation settings will be used. A +ve value is used to specify the maximum number of timesteps."));

TAutoConsoleVariable<float> CVarControlRigPhysicsMaxDeltaTimeOverride(
	TEXT("ControlRig.Physics.MaxDeltaTimeOverride"), -1,
	TEXT("-1 disables the override, so the max delta time authored in the simulation settings will be used. A +ve value is used to specify the maximum delta time."));

constexpr int32 ConstraintChildIndex = 0;
constexpr int32 ConstraintParentIndex = 1;

//======================================================================================================================
FRigPhysicsSolver::FRigPhysicsSolver(const FRigComponentKey& InSolverComponentKey)
	: PhysicsSolverComponentKey(InSolverComponentKey)
{
	WorldObjects = MakeShared<TMap<FWorldObjectKey, FWorldObject>>();
	SimulationMutex = MakeShared<FTransactionallySafeCriticalSection>();
}

//======================================================================================================================
bool FRigPhysicsSolver::ShouldComponentBeInSimulation(
	const URigHierarchy&              Hierarchy, 
	const FRigPhysicsSolverComponent& SolverComponent, 
	const FRigComponentKey&           ComponentKey) const
{
	const FRigPhysicsBodyComponent* PhysicsBodyComponent = Cast<FRigPhysicsBodyComponent>(
		Hierarchy.FindComponent(ComponentKey));

	if (!PhysicsBodyComponent)
	{
		return false;
	}

	check(PhysicsSolverComponentKey == SolverComponent.GetKey());
	if (PhysicsBodyComponent->BodySolverSettings.PhysicsSolverComponentKey == PhysicsSolverComponentKey)
	{
		return true;
	}

	if (!SolverComponent.SolverSettings.bAutomaticallyAddPhysicsBodyComponents)
	{
		return false;
	}

	if (PhysicsBodyComponent->BodySolverSettings.bUseAutomaticSolver)
	{
		FRigElementKey ElementKey = PhysicsBodyComponent->GetElementKey();
		while (ElementKey.IsValid())
		{
			for (FRigComponentKey CK : Hierarchy.GetComponentKeys(ElementKey))
			{
				if (CK == PhysicsSolverComponentKey)
				{
					return true;
				}
			}
			// Note that getting the parent of an element at the root doesn't return the top-level element
			ElementKey = Hierarchy.GetFirstParent(ElementKey);
		}
	}

	return false;
}


//======================================================================================================================
FRigComponentKey FRigPhysicsSolver::FindAncestorBodyInSimulation(
	const URigHierarchy&              Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	const FRigElementKey&             InChildElementKey) const
{
	FRigElementKey AncestorElementKey = Hierarchy.GetFirstParent(InChildElementKey);
	while (AncestorElementKey.IsValid())
	{
		for (const FRigComponentKey& AncestorComponentKey : Hierarchy.GetComponentKeys(AncestorElementKey))
		{
			if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, AncestorComponentKey))
			{
				return AncestorComponentKey;
			}
		}
		AncestorElementKey = Hierarchy.GetFirstParent(AncestorElementKey);
	}
	return FRigComponentKey();
}

//======================================================================================================================
void FRigPhysicsSolver::InitialiseSimulation()
{
	DestroyPhysicsSimulation();

	Simulation = MakeShared<ImmediatePhysics::FSimulation>();

#if CHAOS_SOLVER_DEBUG_NAME
	Simulation->SetDebugName(FName(TEXT("ControlRigPhysics")));
#endif

#if WITH_CHAOS_VISUAL_DEBUGGER
	Simulation->GetChaosVDContextData().Id = FChaosVDRuntimeModule::Get().GenerateUniqueID();
	Simulation->GetChaosVDContextData().Type = static_cast<int32>(EChaosVDContextType::Solver);
#endif

	// This is needed so that when using a fixed timestep, velocities are rewound as well as
	// positions. This is not only more accurate, but it's needed in order to get soft constraint
	// behavior (in particular, for controls) that behave fairly independently of the control-rig
	// tick rate.
	Simulation->SetRewindVelocities(true);

	// Always create a world actor at the origin, for attaching controls to. 
	SimulationActorHandle = CreateBody(
		FName(TEXT("Simulation")), FRigPhysicsCollision(), nullptr, nullptr, FTransform());
}

//======================================================================================================================
void FRigPhysicsSolver::InitialiseControlRecords(
	const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent)
{
	ensure(ControlRecords.IsEmpty());

	TArray<FRigComponentKey> AllComponentKeys = Hierarchy.GetAllComponentKeys();

	for (FRigComponentKey ComponentKey : AllComponentKeys)
	{
		if (const FRigPhysicsControlComponent* ControlComponent = Cast<FRigPhysicsControlComponent>(
			Hierarchy.FindComponent(ComponentKey)))
		{
			// The authored body components may be blank, in which case we need to find the automatic ones.
			// Resolve keys locally, then seed the cached components in the record at the end.
			FRigComponentKey ParentBodyKey = ControlComponent->ParentBodyComponentKey;
			FRigComponentKey ChildBodyKey = ControlComponent->ChildBodyComponentKey;

			// Automate the child
			if (!ChildBodyKey.IsValid())
			{
				TArray<FRigComponentKey> SiblingComponentKeys = Hierarchy.GetComponentKeys(ComponentKey.ElementKey);
				for (FRigComponentKey SiblingComponentKey : SiblingComponentKeys)
				{
					if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, SiblingComponentKey))
					{
						ChildBodyKey = SiblingComponentKey;
						break;
					}
				}
			}

			if (ControlComponent->bUseParentBodyAsDefault)
			{
				if (!ParentBodyKey.IsValid())
				{
					ParentBodyKey =
						FindAncestorBodyInSimulation(Hierarchy, SolverComponent, ComponentKey.ElementKey);
				}
			}

			if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, ChildBodyKey))
			{
				// Here, an invalid parent component key indicates a sim-space control.
				if (!ParentBodyKey.IsValid() ||
					ShouldComponentBeInSimulation(Hierarchy, SolverComponent, ParentBodyKey))
				{
					// Seed the cached lookups so the per-tick fast path can resolve the component
					// pointers without going through the hierarchy TMap.
					FRigControlRecord ControlRecord;
					ControlRecord.CachedPhysicsControlComponent = FCachedRigComponent(ComponentKey, &Hierarchy, true);
					ControlRecord.CachedChildBodyComponent = FCachedRigComponent(ChildBodyKey, &Hierarchy, true);
					if (ParentBodyKey.IsValid())
					{
						ControlRecord.CachedParentBodyComponent = FCachedRigComponent(ParentBodyKey, &Hierarchy, true);
					}
					// Just make the record for now - it will be instantiated later
					ControlRecords.Add(ComponentKey, ControlRecord);
				}
			}
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::InitialiseJointRecords(
	const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent)
{
	ensure(JointRecords.IsEmpty());

	FRigComponentKey SolverComponentKey = SolverComponent.GetKey();

	TArray<FRigComponentKey> AllComponentKeys = Hierarchy.GetAllComponentKeys();

	for (FRigComponentKey ComponentKey : AllComponentKeys)
	{
		// Consider all the joint components in turn
		if (const FRigPhysicsJointComponent* JointComponent = Cast<FRigPhysicsJointComponent>(
			Hierarchy.FindComponent(ComponentKey)))
		{
			// The authored body components may be blank, in which case we need to find the automatic ones.
			// Resolve keys locally, then seed the cached components in the record at the end.
			FRigComponentKey ParentBodyKey = JointComponent->ParentBodyComponentKey;
			FRigComponentKey ChildBodyKey = JointComponent->ChildBodyComponentKey;

			if (!ChildBodyKey.IsValid())
			{
				// Look for a suitable child body
				TArray<FRigComponentKey> SiblingComponentKeys = Hierarchy.GetComponentKeys(ComponentKey.ElementKey);
				for (FRigComponentKey SiblingComponentKey : SiblingComponentKeys)
				{
					if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, SiblingComponentKey))
					{
						ChildBodyKey = SiblingComponentKey;
						break;
					}
				}
			}

			if (!ParentBodyKey.IsValid())
			{
				// Look for a suitable parent body, making sure it is in the same solver
				ParentBodyKey =
					FindAncestorBodyInSimulation(Hierarchy, SolverComponent, ComponentKey.ElementKey);
			}

			if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, ChildBodyKey))
			{
				if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, ParentBodyKey))
				{
					// Seed the cached lookups (see InitialiseControlRecords for rationale).
					FRigJointRecord JointRecord;
					JointRecord.CachedPhysicsJointComponent = FCachedRigComponent(ComponentKey, &Hierarchy, true);
					JointRecord.CachedChildBodyComponent = FCachedRigComponent(ChildBodyKey, &Hierarchy, true);
					JointRecord.CachedParentBodyComponent = FCachedRigComponent(ParentBodyKey, &Hierarchy, true);
					// Just make the record for now - it will be instantiated later
					JointRecords.Add(ComponentKey, JointRecord);
				}
			}
		}
	}
}


//======================================================================================================================
void FRigPhysicsSolver::InitialiseBodyRecords(
	const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent)
{
	ensure(BodyRecords.IsEmpty());

	FRigComponentKey SolverComponentKey = SolverComponent.GetKey();

	TArray<FRigComponentKey> AllComponentKeys = Hierarchy.GetAllComponentKeys();

	// All the components in this simulation
	TArray<FRigComponentKey> UnsortedBodyComponentKeys;

	for (FRigComponentKey ComponentKey : AllComponentKeys)
	{
		if (ShouldComponentBeInSimulation(Hierarchy, SolverComponent, ComponentKey))
		{
			// Just make the record for now - it will be instantiated later
			FRigBodyRecord& Record = BodyRecords.Add(ComponentKey);
			// Seed the cached lookup (see InitialiseControlRecords for rationale).
			Record.CachedPhysicsBodyComponent = FCachedRigComponent(ComponentKey, &Hierarchy, true);
			UnsortedBodyComponentKeys.Add(ComponentKey);
		}
	}

	// Sort the component keys according to the traversal of their element (i.e. from root to leaf)
	SortedBodyComponentKeys.Empty(UnsortedBodyComponentKeys.Num());
	Hierarchy.Traverse([this, &UnsortedBodyComponentKeys](FRigBaseElement* Element, bool& bContinue)
		{
			const FRigElementKey& Key = Element->GetKey();
			for (const FRigComponentKey& ComponentKey : UnsortedBodyComponentKeys)
			{
				if (ComponentKey.ElementKey == Key)
				{
					SortedBodyComponentKeys.Push(ComponentKey);
				}
			}
		});
}

//======================================================================================================================
void FRigPhysicsSolver::DestroyPhysicsSimulation()
{
	if (Simulation.IsValid())
	{
		for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
		{
			FRigBodyRecord& Record = BodyRecordPair.Value;
			if (Record.ActorHandle)
			{
				Simulation->DestroyActor(Record.ActorHandle);
			}
		}

		for (TPair<FRigComponentKey, FRigJointRecord>& JointRecordPair : JointRecords)
		{
			FRigJointRecord& Record = JointRecordPair.Value;
			if (Record.JointHandle)
			{
				Simulation->DestroyJoint(Record.JointHandle);
			}
		}

		for (TPair<FRigComponentKey, FRigControlRecord>& ControlRecordPair : ControlRecords)
		{
			FRigControlRecord& Record = ControlRecordPair.Value;
			if (Record.JointHandle)
			{
				Simulation->DestroyJoint(Record.JointHandle);
			}
		}

		if (CollisionActorHandle)
		{
			Simulation->DestroyActor(CollisionActorHandle);
		}

		if (SimulationActorHandle)
		{
			Simulation->DestroyActor(SimulationActorHandle);
		}
	}

	BodyRecords.Empty();
	JointRecords.Empty();
	ControlRecords.Empty();

	CollisionActorHandle = nullptr;
	SimulationActorHandle = nullptr;

	Simulation.Reset();
}

//======================================================================================================================
static void SetCommonProperties(const FRigPhysicsCollisionShape& Shape, FKShapeElem& ShapeElem)
{
	ShapeElem.RestOffset = Shape.RestOffset;
	ShapeElem.SetName(Shape.Name);
	ShapeElem.SetContributeToMass(Shape.bContributeToMass);
#ifdef PER_SHAPE_COLLISION
	// Note that FKShapeElem supports enabling/disabling collision per shape, but this is discarded by the immediate
	// solver.
	ShapeElem.SetCollisionEnabled(Shape.CollisionEnabled);
#endif
}

//======================================================================================================================
static bool CreateGeometry(
	const FRigPhysicsCollision&               Collision,
	const FRigPhysicsDynamics*                Dynamics,
	const Chaos::FReal                        Density,
	Chaos::FReal&                             OutMass, 
	Chaos::FVec3&                             OutInertia, 
	Chaos::FRigidTransform3&                  OutCoMTransform, 
	Chaos::FImplicitObjectPtr&                OutGeom, 
	TArray<TUniquePtr<Chaos::FPerShapeData>>& OutShapes)
{
	using namespace Chaos;

	OutMass = 0.0f;
	OutInertia = FVector::ZeroVector;
	OutCoMTransform = FTransform::Identity;

	// Set the filter to collide with everything (we use a broad phase that only contains particle
	// pairs that are explicitly set to collide)
	FBodyCollisionData BodyCollisionData;

	const Chaos::Filter::FShapeFilterData ShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(Chaos::EFilterFlags::All);
	BodyCollisionData.CollisionFilterData.ComplexShapeFilterData = BodyCollisionData.CollisionFilterData.SimpleShapeFilterData = ShapeFilter;
	BodyCollisionData.CollisionFilterData.FilterInstanceData = Chaos::Filter::FInstanceData();

	// See FBodyInstance::BuildBodyCollisionFlags
	BodyCollisionData.CollisionFlags.bEnableQueryCollision = false;
	BodyCollisionData.CollisionFlags.bEnableSimCollisionSimple = true;
	BodyCollisionData.CollisionFlags.bEnableSimCollisionComplex = false;
	BodyCollisionData.CollisionFlags.bEnableProbeCollision = false;

	FKAggregateGeom AggGeom;
	for (const FRigPhysicsCollisionBox& Shape : Collision.Boxes)
	{
		FKBoxElem Elem(Shape.Extents.X, Shape.Extents.Y, Shape.Extents.Z);
		SetCommonProperties(Shape, Elem);
		Elem.Center = Shape.TM.GetTranslation();
		Elem.Rotation = Shape.TM.Rotator();
		AggGeom.BoxElems.Add(Elem);
	}

	for (const FRigPhysicsCollisionSphere& Shape : Collision.Spheres)
	{
		FKSphereElem Elem(Shape.Radius);
		SetCommonProperties(Shape, Elem);
		Elem.Center = Shape.TM.GetTranslation();
		// Note that there is no rotation
		AggGeom.SphereElems.Add(Elem);
	}

	for (const FRigPhysicsCollisionCapsule& Shape : Collision.Capsules)
	{
		FKSphylElem Elem(Shape.Radius, Shape.Length);
		SetCommonProperties(Shape, Elem);
		Elem.Center = Shape.TM.GetTranslation();
		Elem.Rotation = Shape.TM.Rotator();
		AggGeom.SphylElems.Add(Elem);
	}

	for (const FRigPhysicsCollisionConvex& Shape : Collision.Convexes)
	{
		// A convex hull needs at least four non-coplanar vertices. ChaosInterface::CreateGeometry
		// silently skips FKConvexElems whose Chaos convex mesh is null.
		if (Shape.VertexData.Num() < 4)
		{
			continue;
		}

		// Add an empty element first and populate it in place. 
		FKConvexElem& Elem = AggGeom.ConvexElems[AggGeom.ConvexElems.AddDefaulted()];
		SetCommonProperties(Shape, Elem);

		// ChaosInterface::CreateGeometry does not honour FKConvexElem::GetTransform() for convex
		// elems (unlike Center/Rotation on box/sphere/capsule), so bake our shape transform into
		// the vertices up-front. The un-transformed data still live in Shape.VertexData / Shape.TM
		// in the component, so edits continue to work. 
		Elem.VertexData.Reserve(Shape.VertexData.Num());
		TArray<Chaos::FConvex::FVec3Type> FloatVerts;
		FloatVerts.Reserve(Shape.VertexData.Num());
		for (const FVector& V : Shape.VertexData)
		{
			const FVector Transformed = Shape.TM.TransformPosition(V);
			Elem.VertexData.Add(Transformed);
			FloatVerts.Add(Chaos::FConvex::FVec3Type(Transformed));
		}
		Elem.UpdateElemBox();
		Elem.SetConvexMeshObject(
			Chaos::FConvexPtr(new Chaos::FConvex(FloatVerts, 0.0f)),
			FKConvexElem::EConvexDataUpdateMethod::UpdateConvexDataOnlyIfMissing);
	}

	FGeometryAddParams AddParams;
	AddParams.CollisionData = BodyCollisionData;
	AddParams.CollisionTraceType = ECollisionTraceFlag::CTF_UseSimpleAsComplex;
	AddParams.Scale = FVector(1, 1, 1);
	AddParams.LocalTransform = FTransform::Identity; // How are these used? We will just set TM afterwards anyway
	AddParams.WorldTransform = FTransform::Identity;
	AddParams.Geometry = &AggGeom;

	TArray<Chaos::FImplicitObjectPtr> Geoms;
	FShapesArray Shapes;
	ChaosInterface::CreateGeometry(AddParams, Geoms, Shapes);

	if (Geoms.Num() == 0)
	{
		return false;
	}

	// Calculate mass properties, if we have dynamics
	if (Dynamics)
	{
		// Whether each shape contributes to mass. It would be easier if ComputeMassProperties knew
		// how to extract this info. Maybe it should be a flag in PerShapeData
		TArray<bool> bContributesToMass;
		bContributesToMass.Reserve(Shapes.Num());
		for (int32 ShapeIndex = 0; ShapeIndex < Shapes.Num(); ++ShapeIndex)
		{
			const TUniquePtr<FPerShapeData>& Shape = Shapes[ShapeIndex];
			const FKShapeElem* ShapeElem = FChaosUserData::Get<FKShapeElem>(Shape->GetUserData());
			bool bHasMass = ShapeElem && ShapeElem->GetContributeToMass();
			bContributesToMass.Add(bHasMass);
		}

		Chaos::FMassProperties MassProperties;
		ChaosInterface::CalculateMassPropertiesFromShapeCollection(
			MassProperties, Shapes, bContributesToMass, Density);

		OutMass = MassProperties.Mass;
		OutInertia = MassProperties.InertiaTensor.GetDiagonal();
		OutCoMTransform = FTransform(MassProperties.RotationOfMass, MassProperties.CenterOfMass);
	}

	// If we have multiple root shapes, wrap them in a union
	if (Geoms.Num() == 1)
	{
		OutGeom = MoveTemp(Geoms[0]);
	}
	else
	{
		OutGeom = MakeImplicitObjectPtr<FImplicitObjectUnion>(MoveTemp(Geoms));
	}

	for (TUniquePtr<FPerShapeData>& Shape : Shapes)
	{
		OutShapes.Emplace(MoveTemp(Shape));
	}

	return true;
}

//======================================================================================================================
ImmediatePhysics::FActorHandle* FRigPhysicsSolver::CreateBody(
	const FName                        BodyName,
	const FRigPhysicsCollision&        Collision,
	const FRigPhysicsDynamics*         Dynamics,
	const FPhysicsControlModifierData* BodyData,
	const FTransform&                  BodyRelSimSpaceTM) const
{
	ImmediatePhysics::FActorSetup ActorSetup;

	if (Dynamics)
	{
		ActorSetup.ActorType = ImmediatePhysics::EActorType::DynamicActor;
		ActorSetup.bEnableGravity = true;
		// Damping is assigned below, after Mass has been resolved (it may be scaled by inverse mass).
	}
	else
	{
		ActorSetup.ActorType = ImmediatePhysics::EActorType::KinematicActor;
	}

	if (BodyData)
	{
		ActorSetup.bUpdateKinematicFromSimulation = BodyData->bUpdateKinematicFromSimulation;
	}
	else
	{
		ActorSetup.bUpdateKinematicFromSimulation = false;
	}

	Chaos::FVec3 Inertia;
	Chaos::FRigidTransform3 CoMTransform;
	Chaos::FReal Mass;
	Chaos::FImplicitObjectPtr BodyGeom;
	TArray<TUniquePtr<Chaos::FPerShapeData>> BodyShapes;
	Chaos::FReal Density = Dynamics ? Dynamics->Density : 1.0f;
	// Convert from g/cm^3 to kg/cm^3 - slightly odd units, but when we come to calculate density,
	// distances will be in cm and masses are in kg. See CalculateMassPropertiesFromShapeCollection
	Density *= 1e-3;

	bool bGeometryCreated = CreateGeometry(
		Collision, Dynamics, Density, Mass, Inertia, CoMTransform, BodyGeom, BodyShapes);

	// We will have created with an arbitrary density - adjust to result in the desired mass.
	ActorSetup.Mass = Mass;
	ActorSetup.Inertia = Inertia;
	ActorSetup.Transform = BodyRelSimSpaceTM;
	ActorSetup.CoMTransform = CoMTransform;
	ActorSetup.Geometry = MoveTemp(BodyGeom);
	ActorSetup.Shapes = MoveTemp(BodyShapes);

	if (Dynamics)
	{
		if (Mass > 0.0f && Dynamics->MassOverride > 0.0f)
		{
			ActorSetup.Mass = Dynamics->MassOverride;
			ActorSetup.Inertia = (Dynamics->MassOverride / Mass) * Inertia;
		}
		if (Dynamics->bOverrideMomentsOfInertia)
		{
			ActorSetup.Inertia = Dynamics->MomentsOfInertiaOverride;
		}
		if (Dynamics->bOverrideCentreOfMass)
		{
			ActorSetup.CoMTransform.SetLocation(Dynamics->CentreOfMassOverride);
		}
		ActorSetup.CoMTransform.AddToTranslation(Dynamics->CentreOfMassNudge);

		// Apply damping after Mass has been resolved (MassOverride / density). When
		// bScaleDampingByInverseMass is set, scale by inverse mass for drag-like behaviour.
		const float DampingScale = Dynamics->bScaleDampingByInverseMass
			? ((ActorSetup.Mass > UE_SMALL_NUMBER) ? 1.0f / static_cast<float>(ActorSetup.Mass) : 0.0f)
			: 1.0f;
		ActorSetup.LinearDamping = Dynamics->LinearDamping * DampingScale;
		ActorSetup.AngularDamping = Dynamics->AngularDamping * DampingScale;
	}


	ActorSetup.Material = MakeUnique<Chaos::FChaosPhysicsMaterial>();
	ActorSetup.Material->Friction = Collision.Material.Friction;
	ActorSetup.Material->StaticFriction = Collision.Material.Friction;
	ActorSetup.Material->Restitution = Collision.Material.Restitution;
	ActorSetup.Material->FrictionCombineMode =
		(Chaos::FChaosPhysicsMaterial::ECombineMode)Collision.Material.FrictionCombineMode;
	ActorSetup.Material->RestitutionCombineMode =
		(Chaos::FChaosPhysicsMaterial::ECombineMode)Collision.Material.RestitutionCombineMode;

	ImmediatePhysics::FActorHandle* ActorHandle = Simulation->CreateActor(MoveTemp(ActorSetup));
	if (!ActorHandle)
	{
		UE_LOGF(LogRigPhysics, Warning,
			"Unable to create body %ls", *BodyName.ToString());
		return nullptr;
	}

	ActorHandle->SetName(BodyName);
#if CHAOS_DEBUG_NAME
	if (Chaos::FGeometryParticleHandle* ParticleHandle = ActorHandle->GetParticle())
	{
		ParticleHandle->SetDebugName(MakeShared<FString>(BodyName.ToString()));
	}
#endif

	if (bGeometryCreated)
	{
		Simulation->AddToCollidingPairs(ActorHandle);
		if (Dynamics)
		{
			// Note that particles are always created disabled. They will simulate when disabled,
			// but won't collide!
			ActorHandle->SetEnabled(true);
		}
	}
	else
	{
		Simulation->SetHasCollision(ActorHandle, false);
	}

	return ActorHandle;
}

//======================================================================================================================
void FRigPhysicsSolver::Instantiate(
	const FRigVMExecuteContext&       ExecuteContext, 
	URigHierarchy&                    Hierarchy, 
	const FRigPhysicsSolverComponent& SolverComponent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_Instantiate);

	if (!bNeedToInstantiate)
	{
		return;
	}

	// Lock against access to the simulation, in case of access by the WorldObject task
	UE::TScopeLock<FTransactionallySafeCriticalSection> Lock(*SimulationMutex.Get());

	// We need the simulation space in order to instantiate properly. This is not ideal, as we may
	// end up updating the simulation space data twice (thus inserting data into the history). This
	// shouldn't really matter as it will only be on the first step.
	UpdateSimulationSpaceStateAndCalculateData(
		ExecuteContext, Hierarchy, SolverComponent, 0.0f, ExecuteContext.GetAbsoluteTime());

	InitialiseSimulation();

	InitialiseBodyRecords(Hierarchy, SolverComponent);

	InitialiseJointRecords(Hierarchy, SolverComponent);

	InitialiseControlRecords(Hierarchy, SolverComponent);

	FRigPhysicsIgnorePairs IgnorePairs;

	InstantiatePhysicsBodies(Hierarchy, SolverComponent, IgnorePairs);

	InstantiatePhysicsJoints(Hierarchy, SolverComponent, IgnorePairs);

	InstantiateControls(Hierarchy, SolverComponent, IgnorePairs);

	// This is done last as it applies IgnorePairs
	InstantiateSolverCollision(SolverComponent, IgnorePairs);

	bNeedToInstantiate = false;
}

//======================================================================================================================
ImmediatePhysics::FActorHandle* FRigPhysicsSolver::GetActor(const FRigComponentKey& ComponentKey) const
{
	if (const FRigBodyRecord* BodyRecord = BodyRecords.Find(ComponentKey))
	{
		return BodyRecord->ActorHandle;
	}
	// Also check to see if it is our own (the solver shapes)
	if (ComponentKey == PhysicsSolverComponentKey)
	{
		return SimulationActorHandle;
	}
	return nullptr;
}

//======================================================================================================================
void FRigPhysicsSolver::InstantiateSolverCollision(
	const FRigPhysicsSolverComponent& SolverComponent,
	FRigPhysicsIgnorePairs&           IgnorePairs)
{
	// Optionally create an object to contain environment collision
	if (!SolverComponent.SolverSettings.Collision.IsEmpty())
	{
		// When we make these additional collision shapes, their actors are all considered to be at
		// the origin, with the offsets being contained in the collision shapes.
		FTransform BodyRelSimSpaceTM = ConvertCollisionSpaceTransformToSimSpace(
			SolverComponent.SolverSettings, FTransform());

		CollisionActorHandle = CreateBody(
			FName(TEXT("Environment")), SolverComponent.SolverSettings.Collision, nullptr, nullptr, BodyRelSimSpaceTM);
	}

	// Add no-collision pairs
	TArray<ImmediatePhysics_Chaos::FSimulation::FIgnorePair> ChaosIgnorePairs;

	for (const FRigPhysicsIgnorePair& IgnorePair : IgnorePairs)
	{
		ImmediatePhysics::FSimulation::FIgnorePair ChaosPair;
		ChaosPair.A = GetActor(IgnorePair.A);
		ChaosPair.B = GetActor(IgnorePair.B);
		if (ChaosPair.A && ChaosPair.B)
		{
			ChaosIgnorePairs.Add(ChaosPair);
		}
	}
	Simulation->SetIgnoreCollisionPairTable(ChaosIgnorePairs);

}

//======================================================================================================================
void FRigPhysicsSolver::InstantiatePhysicsBodies(
	URigHierarchy&                    Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	FRigPhysicsIgnorePairs&           IgnorePairs)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_InstantiatePhysicsBodies);

	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		const FRigComponentKey& ComponentKey = BodyRecordPair.Key;
		FRigBodyRecord& Record = BodyRecordPair.Value;

		if (FRigPhysicsBodyComponent* PhysicsBodyComponent = Cast<FRigPhysicsBodyComponent>(
			Hierarchy.FindComponent(ComponentKey)))
		{
			const FRigPhysicsCollision& Collision = PhysicsBodyComponent->Collision;
			const FRigPhysicsDynamics& Dynamics = PhysicsBodyComponent->Dynamics;
			const FPhysicsControlModifierData& BodyData = PhysicsBodyComponent->BodyData;

			FRigElementKey SourceKey = PhysicsBodyComponent->BodySolverSettings.SourceBone;
			if (!SourceKey.IsValid())
			{
				SourceKey = ComponentKey.ElementKey;
			}

			// What should we do if the key is not valid? 
			if (SourceKey.IsValid())
			{
				Record.FinalComponentSpaceTM = Hierarchy.GetGlobalTransform(SourceKey);
				const FTransform SourceSimulationSpaceTM =
					ConvertComponentSpaceTransformToSimSpace(
						SolverComponent.SolverSettings, Record.FinalComponentSpaceTM);
				Record.ActorHandle = CreateBody(
					ComponentKey.ElementKey.Name, Collision, &Dynamics, &BodyData, SourceSimulationSpaceTM);
				if (Record.ActorHandle)
				{
					SetPhysicsBodyComponentState(SolverSettings, Record, *PhysicsBodyComponent);
				}
			}

			Record.TargetElementKey = PhysicsBodyComponent->BodySolverSettings.TargetBone;
			if (!Record.TargetElementKey.IsValid())
			{
				Record.TargetElementKey = ComponentKey.ElementKey;
			}

			for (const FRigComponentKey& NoCollisionKey : PhysicsBodyComponent->NoCollisionBodies)
			{
				IgnorePairs.Add(FRigPhysicsIgnorePair(ComponentKey, NoCollisionKey));
			}
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::SetPhysicsBodyComponentState(
	const FRigPhysicsSolverSettings& SolverSettings,
	FRigBodyRecord&                  Record, 
	FRigPhysicsBodyComponent&        PhysicsBodyComponent) const
{
	PhysicsBodyComponent.Transform = Record.FinalComponentSpaceTM;
	PhysicsBodyComponent.CoMTransform = Record.ActorHandle->GetLocalCoMTransform() * Record.FinalComponentSpaceTM;
	PhysicsBodyComponent.LinearVelocity = 
		ConvertSimSpaceVectorToComponentSpace(SolverSettings, Record.ActorHandle->GetLinearVelocity());
	PhysicsBodyComponent.AngularVelocity = 
		ConvertSimSpaceVectorToComponentSpace(SolverSettings, Record.ActorHandle->GetAngularVelocity());
}

//======================================================================================================================
void FRigPhysicsSolver::InstantiatePhysicsJoints(
	const URigHierarchy&              Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	FRigPhysicsIgnorePairs&           IgnorePairs)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_InstantiatePhysicsJoints);

	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	// Once all the bodies are created, we can make their physics joints
	for (TPair<FRigComponentKey, FRigJointRecord>& JointRecordPair : JointRecords)
	{
		FRigJointRecord& JointRecord = JointRecordPair.Value;

		if (const FRigPhysicsJointComponent* PhysicsJointComponent =
			GetPhysicsJoint(Hierarchy, JointRecord.CachedPhysicsJointComponent))
		{
			const FRigPhysicsJointData& JointData = PhysicsJointComponent->JointData;

			const FRigComponentKey ChildBodyComponentKey = JointRecord.CachedChildBodyComponent.GetComponentKey();
			const FRigElementKey ChildBoneKey = ChildBodyComponentKey.ElementKey;

			const FRigComponentKey ParentBodyComponentKey = JointRecord.CachedParentBodyComponent.GetComponentKey();
			const FRigElementKey ParentBoneKey = ParentBodyComponentKey.ElementKey;

			// Joints require both parent and child to exist
			const FRigPhysicsBodyComponent* ChildPhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, JointRecord.CachedChildBodyComponent);
			FRigBodyRecord* ChildBodyRecord = BodyRecords.Find(ChildBodyComponentKey);
			ImmediatePhysics::FActorHandle* ChildActorHandle = ChildBodyRecord ? ChildBodyRecord->ActorHandle : nullptr;
			if (!ChildActorHandle)
			{
				continue;
			}

			const FRigPhysicsBodyComponent* ParentPhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, JointRecord.CachedParentBodyComponent);
			FRigBodyRecord* ParentBodyRecord = BodyRecords.Find(ParentBodyComponentKey);
			ImmediatePhysics::FActorHandle* ParentActorHandle = ParentBodyRecord ? ParentBodyRecord->ActorHandle : nullptr;
			if (!ParentActorHandle)
			{
				continue;
			}

			// Make the physics joint (joint constraint). 
			// UE likes to treat Body1 (index 0) as the child, and Body2 (index 1) as the parent
			{
				const FTransform ParentCoMTransform = ParentActorHandle->GetLocalCoMTransform();
				const FTransform ChildCoMTransform = ChildActorHandle->GetLocalCoMTransform();

				Chaos::FPBDJointSettings JointSettings;
				if (JointData.bAutoCalculateChildOffset)
				{
					JointSettings.ConnectorTransforms[ConstraintChildIndex] = FTransform();
				}
				JointSettings.ConnectorTransforms[ConstraintChildIndex] =
					JointData.ExtraChildOffset * JointSettings.ConnectorTransforms[ConstraintChildIndex];

				if (JointData.bAutoCalculateParentOffset)
				{
					// When auto-calculating, the transform stored in the parent frame should place
					// it at the child bone's location.
					JointSettings.ConnectorTransforms[ConstraintParentIndex] = 
						Hierarchy.GetLocalTransform(ChildBoneKey, true);
				}
				JointSettings.ConnectorTransforms[ConstraintParentIndex] =
					JointData.ExtraParentOffset * JointSettings.ConnectorTransforms[ConstraintParentIndex];

				ImmediatePhysics::UpdateJointSettingsFromLinearConstraint(JointData.LinearConstraint, JointSettings);
				ImmediatePhysics::UpdateJointSettingsFromConeConstraint(JointData.ConeConstraint, JointSettings);
				ImmediatePhysics::UpdateJointSettingsFromTwistConstraint(JointData.TwistConstraint, JointSettings);

				// The physics setting is backwards, because we can't enable collision on bodies that
				// are set to not collide for other reasons.
				JointSettings.bCollisionEnabled = !JointData.bDisableCollision;
				JointSettings.bProjectionEnabled = 
					JointData.LinearProjectionAmount > 0.0f || JointData.AngularProjectionAmount > 0.0f;
				JointSettings.AngularProjection = JointData.AngularProjectionAmount;
				JointSettings.LinearProjection = JointData.LinearProjectionAmount;
				JointSettings.ParentInvMassScale = JointData.ParentInverseMassScale;

				JointSettings.bUseLinearSolver = SolverSettings.bUseLinearJointSolver;

				JointRecord.JointHandle = Simulation->CreateJoint(
					ImmediatePhysics::FJointSetup(JointSettings, ChildActorHandle, ParentActorHandle));

				if (JointRecord.JointHandle)
				{
					if (Chaos::FPBDJointConstraintHandle* Constraint = JointRecord.JointHandle->GetConstraint())
					{
						Chaos::FPBDJointSettings Settings = Constraint->GetSettings();
						if (!Settings.bCollisionEnabled)
						{
							IgnorePairs.Add(FRigPhysicsIgnorePair(
								JointRecord.CachedChildBodyComponent.GetComponentKey(),
								JointRecord.CachedParentBodyComponent.GetComponentKey()));
						}
					}
				}
			}
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::InstantiateControls(
	const URigHierarchy&              Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	FRigPhysicsIgnorePairs&           IgnorePairs)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_InstantiateControls);

	if (!Simulation.IsValid())
	{
		return;
	}

	for (TPair<FRigComponentKey, FRigControlRecord>& ControlRecordPair : ControlRecords)
	{
		FRigControlRecord& ControlRecord = ControlRecordPair.Value;

		if (const FRigPhysicsControlComponent* ControlComponent =
			GetPhysicsControl(Hierarchy, ControlRecord.CachedPhysicsControlComponent))
		{
			if (FRigBodyRecord* ChildBodyRecord = BodyRecords.Find(ControlRecord.CachedChildBodyComponent.GetComponentKey()))
			{
				FRigBodyRecord* ParentBodyRecord = BodyRecords.Find(ControlRecord.CachedParentBodyComponent.GetComponentKey());

				// ParentBodyRecord being nullptr just means to create a sim-space control. However,
				// we never want to make a sim-space control if the user has set
				// bUseParentBodyAsDefault, because that flag implies the user wanted a parent-space
				// control. Note that when creating chains etc it's quite easy to "accidentally"
				// create a parent-space control for the first body in the chain, even when one
				// isn't actually required. In fact, this can happen very easily with modular rigs
				// because chains may be created independently, and we'll only find out we want a
				// parent-space control on the arm (say) after the spine has been created.
				if (!ParentBodyRecord && ControlComponent->bUseParentBodyAsDefault)
				{
					// We will have a control in the component, but it will do nothing as there will
					// be no physics joint/constraint.
					continue;
				}

				ImmediatePhysics::FActorHandle* const ChildBodyHandle = ChildBodyRecord->ActorHandle;
				ImmediatePhysics::FActorHandle* const ParentBodyHandle = 
					ParentBodyRecord ? ParentBodyRecord->ActorHandle : SimulationActorHandle;

				if (ChildBodyHandle && ParentBodyHandle)
				{
					// The constraint is created disabled - it will be updated in pre-physics
					ControlRecord.JointHandle = CreatePhysicsJoint(*Simulation.Get(), *ChildBodyHandle, *ParentBodyHandle);
				}

				if (!ControlRecord.JointHandle)
				{
					UE_LOGF(LogRigPhysics, Warning,
						"Unable to create control constraint for %ls", 
						*ControlComponent->GetKey().ToString());
				}

				if (ControlComponent->ControlData.bDisableCollision)
				{
					IgnorePairs.Add(FRigPhysicsIgnorePair(
						ControlRecord.CachedChildBodyComponent.GetComponentKey(),
						ControlRecord.CachedParentBodyComponent.GetComponentKey()));
				}
			}
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdateBodyRecordPrePhysics(
	const URigHierarchy&              Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	const float                       DeltaTime,
	FRigBodyRecord&                   Record, 
	const FRigPhysicsBodyComponent*   PhysicsBodyComponent)
{
	if (!Record.ActorHandle)
	{
		return;
	}
	const FRigComponentKey ComponentKey = PhysicsBodyComponent->GetKey();

	// Shuffle the record data
	Record.PrevSourceComponentSpaceVelocity = Record.SourceComponentSpaceVelocity;
	Record.PrevSourceComponentSpaceAngularVelocity = Record.SourceComponentSpaceAngularVelocity;
	Record.PrevSourceComponentSpaceTM = Record.SourceComponentSpaceTM;

	FRigElementKey SourceKey = PhysicsBodyComponent->BodySolverSettings.SourceBone;
	if (!SourceKey.IsValid())
	{
		SourceKey = ComponentKey.ElementKey;
	}
	if (SourceKey.IsValid())
	{
		Record.SourceComponentSpaceTM = Hierarchy.GetGlobalTransform(SourceKey);
	}

	if (UpdateCounter == PreviousUpdateCounter + 1)
	{
		Record.SourceComponentSpaceVelocity = UE::PhysicsControl::CalculateLinearVelocity(
			Record.PrevSourceComponentSpaceTM.GetTranslation(), Record.SourceComponentSpaceTM.GetTranslation(), DeltaTime);
		Record.SourceComponentSpaceAngularVelocity = UE::PhysicsControl::CalculateAngularVelocity(
			Record.PrevSourceComponentSpaceTM.GetRotation(), Record.SourceComponentSpaceTM.GetRotation(), DeltaTime);
	}
	else
	{
		Record.PrevSourceComponentSpaceTM = Record.SourceComponentSpaceTM;
		Record.SourceComponentSpaceVelocity = FVector::ZeroVector;
		Record.SourceComponentSpaceAngularVelocity = FVector::ZeroVector;
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdateBodyRecordsPrePhysics(
	const URigHierarchy&              Hierarchy,
	const FRigPhysicsSolverComponent& SolverComponent,
	const float                       DeltaTime)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateBodyRecordsPrePhysics);
	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		FRigBodyRecord& Record = BodyRecordPair.Value;
		if (const FRigPhysicsBodyComponent* PhysicsBodyComponent =
			GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
		{
			UpdateBodyRecordPrePhysics(Hierarchy, SolverComponent, DeltaTime, Record, PhysicsBodyComponent);
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::ClearBodyForceAndTorques(URigHierarchy& Hierarchy)
{
	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		FRigBodyRecord& Record = BodyRecordPair.Value;
		if (FRigPhysicsBodyComponent* PhysicsBodyComponent =
			GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
		{
			PhysicsBodyComponent->ForceAndTorques.Empty();
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::ResetPoseFromAnimation(
	const FRigPhysicsSolverComponent& SolverComponent,
	const bool                        bUseSourceVelocity)
{
	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		FRigBodyRecord& Record = BodyRecordPair.Value;
		if (!Record.ActorHandle || Record.ActorHandle->GetIsKinematic())
		{
			// Kinematic bodies are driven by their kinematic targets each step, so teleporting
			// here would fight that. The simulation step handles them.
			continue;
		}

		const FTransform SourceSimSpaceTM = ConvertComponentSpaceTransformToSimSpace(
			SolverSettings, Record.SourceComponentSpaceTM);
		Record.ActorHandle->SetWorldTransform(SourceSimSpaceTM);

		if (bUseSourceVelocity)
		{
			const FVector V = ConvertComponentSpaceVectorToSimSpace(
				SolverSettings, Record.SourceComponentSpaceVelocity);
			const FVector W = ConvertComponentSpaceVectorToSimSpace(
				SolverSettings, Record.SourceComponentSpaceAngularVelocity);
			Record.ActorHandle->SetLinearVelocity(V);
			Record.ActorHandle->SetAngularVelocity(W);
		}
		else
		{
			Record.ActorHandle->SetLinearVelocity(FVector::ZeroVector);
			Record.ActorHandle->SetAngularVelocity(FVector::ZeroVector);
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdateBodyPrePhysics(
	const FRigVMExecuteContext&       ExecuteContext, 
	const FRigPhysicsSolverComponent& SolverComponent,
	const FRigBodyRecord&             Record, 
	const FRigPhysicsBodyComponent&   PhysicsBodyComponent)
{
	if (!Record.ActorHandle)
	{
		return;
	}
	const FRigComponentKey ComponentKey = PhysicsBodyComponent.GetKey();

	FPhysicsControlModifierData BodyData = PhysicsBodyComponent.BodyData;
	EPhysicsControlKinematicTargetSpace KinematicTargetSpace = BodyData.KinematicTargetSpace;

	if (SolverComponent.TrackInputCounter > 0)
	{
		// If the body is already kinematic, then we don't want to make it start ignoring a target.
		// However, dynamic objects should avoid using a target if they are just tracking due to a reset.
		if (BodyData.MovementType != EPhysicsMovementType::Kinematic)
		{
			BodyData.MovementType = EPhysicsMovementType::Kinematic;
			KinematicTargetSpace = EPhysicsControlKinematicTargetSpace::IgnoreTarget;
		}
	}

	UpdateBodyFromModifierData(
		*Record.ActorHandle, *Simulation.Get(), BodyData, SimulationSpaceData.Gravity);

	if (Record.ActorHandle->GetIsKinematic())
	{
		// Get the target in component space, and then convert it into sim space if necessary.

		// If the target is already in component space, then that's all we need
		FTransform KinematicTargetCS;
		const FTransform KinematicTargetTM = PhysicsBodyComponent.KinematicTarget;
		switch (KinematicTargetSpace)
		{
		case EPhysicsControlKinematicTargetSpace::Component:
		{
			KinematicTargetCS = KinematicTargetTM;
			break;
		}
		case EPhysicsControlKinematicTargetSpace::World:
		{
			KinematicTargetCS = KinematicTargetTM.GetRelativeTransform(SimulationSpaceState.ComponentTM);
			break;
		}
		default:
		{
			// All the other options are relative to a bone, so the first task is to get
			// that, which will be in component space
			FRigElementKey SourceKey = PhysicsBodyComponent.BodySolverSettings.SourceBone;
			if (!SourceKey.IsValid())
			{
				SourceKey = ComponentKey.ElementKey;
			}
			if (SourceKey.IsValid())
			{
				switch (KinematicTargetSpace)
				{
				case EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace:
				{
					KinematicTargetCS = KinematicTargetTM * Record.SourceComponentSpaceTM;
					break;
				}
				case EPhysicsControlKinematicTargetSpace::OffsetInWorldSpace:
				{
					const FTransform ComponentTM(SimulationSpaceState.ComponentTM);
					// Convert the bone to WS (it's really component space!), apply the target, and convert back. 
					FTransform KinematicTargetWS = Record.SourceComponentSpaceTM * ComponentTM;
					// Note that we don't want to rotate the translation by the target - we apply it
					// individually to orientation and position.
					KinematicTargetWS.AddToTranslation(KinematicTargetTM.GetTranslation());
					KinematicTargetWS.SetRotation(KinematicTargetTM.GetRotation() * KinematicTargetWS.GetRotation());
					KinematicTargetCS = KinematicTargetWS.GetRelativeTransform(ComponentTM);
					break;
				}
				case EPhysicsControlKinematicTargetSpace::OffsetInComponentSpace:
				{
					// This applies the offset as a transform in component space. Note that this is
					// different to what PCC does, because in ControlRig "world" is really
					// "component" space.
					KinematicTargetCS =  Record.SourceComponentSpaceTM * KinematicTargetTM;
					break;
				}
				case EPhysicsControlKinematicTargetSpace::IgnoreTarget:
				{
					KinematicTargetCS = Record.SourceComponentSpaceTM;
					break;
				}
				default:
				{
					KinematicTargetCS = Record.SourceComponentSpaceTM;
					UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("Kinematic target space is not valid"));
				}
				}
			}
		}
		}
		const FTransform KinematicTargetSimSpaceTM = ConvertComponentSpaceTransformToSimSpace(
			SolverComponent.SolverSettings, KinematicTargetCS);
		Record.ActorHandle->SetKinematicTarget(KinematicTargetSimSpaceTM);
	}
	else
	{
		// TODO move damping into BodyData - any PhysicsControl system should be able to use it
		const FRigPhysicsDynamics& Dynamics = PhysicsBodyComponent.Dynamics;
		const float DampingScale = Dynamics.bScaleDampingByInverseMass
			? static_cast<float>(Record.ActorHandle->GetInverseMass())
			: 1.0f;
		Record.ActorHandle->SetLinearDamping(Dynamics.LinearDamping * DampingScale);
		Record.ActorHandle->SetAngularDamping(Dynamics.AngularDamping * DampingScale);
	}
}

//======================================================================================================================
void FRigPhysicsSolver::ApplyForcesPrePhysics(
	const FRigVMExecuteContext&       ExecuteContext,
	const FRigPhysicsSolverComponent& SolverComponent,
	const FRigBodyRecord&             Record,
	const FRigPhysicsBodyComponent&   PhysicsBodyComponent)
{
	if (PhysicsBodyComponent.ForceAndTorques.IsEmpty() || !Record.ActorHandle || Record.ActorHandle->GetIsKinematic())
	{
		return;
	}

	ImmediatePhysics::FActorHandle& ActorHandle = *Record.ActorHandle;
	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	for (const FPhysicsControlNamedForceAndTorqueData& Data : PhysicsBodyComponent.ForceAndTorques)
	{
		const FPhysicsControlForceAndTorqueData& ForceAndTorqueData = Data.ForceAndTorqueData;

		// Need to convert the requested force/torque into simulation space first
		FVector LinearSS;
		FVector AngularSS;

		FTransform BodyTM = ActorHandle.GetWorldTransform();

		switch (ForceAndTorqueData.ForceAndTorqueSpace)
		{
		case EPhysicsControlSpace::World: // For Control Rig, World and Component space are considered to be the same
		case EPhysicsControlSpace::Component:
			LinearSS = ConvertComponentSpaceVectorToSimSpace(
				SolverSettings, SimulationSpaceState.ComponentTM.InverseTransformVector(ForceAndTorqueData.Linear));
			AngularSS = ConvertComponentSpaceVectorToSimSpace(
				SolverSettings, SimulationSpaceState.ComponentTM.InverseTransformVector(ForceAndTorqueData.Angular));
			break;
		case EPhysicsControlSpace::Body:
			LinearSS = BodyTM.GetRotation() * ForceAndTorqueData.Linear;
			AngularSS = BodyTM.GetRotation() * ForceAndTorqueData.Angular;
			break;
		default:
			LinearSS = FVector::ZeroVector;
			AngularSS = FVector::ZeroVector;
			break;
		}

		// Apply it differently depending on the location
		ImmediatePhysics_Shared::EForceType FT = ConvertForceType(ForceAndTorqueData.Type);
		switch (ForceAndTorqueData.LocationSpace)
		{
		case EPhysicsControlSpace::World: // For Control Rig, World and Component space are considered to be the same
		case EPhysicsControlSpace::Component:
		{
			FVector SimSpaceLocation = ConvertWorldPositionToSimSpaceNoScale(
				SolverSettings, ForceAndTorqueData.Location);
			ActorHandle.AddForceAtLocation(LinearSS, SimSpaceLocation, FT);
		}
		break;
		case EPhysicsControlSpace::Body:
		{
			FVector SimSpaceLocation = ActorHandle.GetWorldTransform().TransformPositionNoScale(
				ActorHandle.GetLocalCoMLocation() + ForceAndTorqueData.Location);
			ActorHandle.AddForceAtLocation(LinearSS, SimSpaceLocation, FT);
		}
		break;
		default:
			break;
		}
		// Torque doesn't get applied at a location
		ActorHandle.AddTorque(AngularSS, FT);
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdateJointPrePhysics(
	const URigHierarchy&                Hierarchy, 
	FRigJointRecord&                    Record, 
	const FRigPhysicsJointComponent&    PhysicsJointComponent, 
	const float                         DeltaTime)
{
	// Now update the joint targets
	if (!Record.JointHandle)
	{
		return;
	}
	if (Chaos::FPBDJointConstraintHandle* Constraint = Record.JointHandle->GetConstraint())
	{
		const FRigComponentKey ComponentKey = PhysicsJointComponent.GetKey();

		// Set the drive strength etc
		const FRigPhysicsJointData& JointData = PhysicsJointComponent.JointData;
		const FRigPhysicsDriveData& DriveData = PhysicsJointComponent.DriveData;

		if (PhysicsJointComponent.DirtyFlags != ERigPhysicsJointComponentDirtyFlags::None)
		{
			Chaos::FPBDJointSettings Settings = Constraint->GetSettings();

			if ((PhysicsJointComponent.DirtyFlags & ERigPhysicsJointComponentDirtyFlags::Joint) != 
				ERigPhysicsJointComponentDirtyFlags::None)
			{
				SetPhysicsJointEnabled(*Record.JointHandle, JointData.bEnabled);

				ImmediatePhysics::UpdateJointSettingsFromLinearConstraint(JointData.LinearConstraint, Settings);
				ImmediatePhysics::UpdateJointSettingsFromConeConstraint(JointData.ConeConstraint, Settings);
				ImmediatePhysics::UpdateJointSettingsFromTwistConstraint(JointData.TwistConstraint, Settings);
			}

			if ((PhysicsJointComponent.DirtyFlags & ERigPhysicsJointComponentDirtyFlags::Drive) !=
				ERigPhysicsJointComponentDirtyFlags::None)
			{
				ImmediatePhysics::UpdateJointSettingsFromLinearDriveConstraint(DriveData.LinearDriveConstraint, Settings);
				ImmediatePhysics::UpdateJointSettingsFromAngularDriveConstraint(DriveData.AngularDriveConstraint, Settings);
			}

			Constraint->SetSettings(Settings);

			PhysicsJointComponent.DirtyFlags = ERigPhysicsJointComponentDirtyFlags::None;
		}

		const Chaos::FPBDJointSettings& Settings = Constraint->GetSettings();

		// Now set the actual target
		if (JointData.bEnabled && (
			Settings.AngularDriveStiffness.SquaredLength() > 0.0f ||
			Settings.AngularDriveDamping.SquaredLength() > 0.0f ||
			Settings.LinearDriveStiffness.SquaredLength() > 0.0f ||
			Settings.LinearDriveDamping.SquaredLength() > 0.0f))
		{
			// Multiplier on the velocity calculated from the current and previous target
			//
			// TODO surely this should always be taken from DriveData? Do we/can we
			// distinguish between manual and animation velocities?
			float TargetVelocityMultiplier = 1.0f;

			FRigElementKey ChildSourceKey;
			if (const FRigPhysicsBodyComponent* ChildPhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedChildBodyComponent))
			{
				ChildSourceKey = ChildPhysicsBodyComponent->BodySolverSettings.SourceBone;
				if (!ChildSourceKey.IsValid())
				{
					ChildSourceKey = ComponentKey.ElementKey;
				}
				TargetVelocityMultiplier = DriveData.SkeletalAnimationVelocityMultiplier;
			}

			FTransform DriveTargetTM;
			if (DriveData.bUseSkeletalAnimation)
			{
				FRigElementKey ParentSourceKey;
				if (const FRigPhysicsBodyComponent* ParentPhysicsBodyComponent =
					GetPhysicsBody(Hierarchy, Record.CachedParentBodyComponent))
				{
					ParentSourceKey = ParentPhysicsBodyComponent->BodySolverSettings.SourceBone;
					if (!ParentSourceKey.IsValid())
					{
						ParentSourceKey = Record.CachedParentBodyComponent.GetElementKey();
					}
				}

				if (!ChildSourceKey.IsValid() || !ParentSourceKey.IsValid())
				{
					return;
				}

				// TODO now all transforms are being cached in the record, get them from there
				// rather than the hierarchy

				// Note that the drive operates between a parent and child part, so we
				// don't need to worry about global/component (etc) space.
				const FTransform ChildTM(Hierarchy.GetGlobalTransform(ChildSourceKey));
				const FTransform ParentTM(Hierarchy.GetGlobalTransform(ParentSourceKey));

				const FTransform ComponentSpaceParentFrameTM = 
					Settings.ConnectorTransforms[ConstraintParentIndex] * ParentTM;
				const FTransform ComponentSpaceChildFrameTM = 
					Settings.ConnectorTransforms[ConstraintChildIndex] * ChildTM;

				DriveTargetTM = ComponentSpaceChildFrameTM.GetRelativeTransform(ComponentSpaceParentFrameTM);
			}

			// Apply the offset in the frame of the (potentially animation) target
			DriveTargetTM = FTransform(				
				DriveData.AngularDriveConstraint.OrientationTarget.Quaternion(),
				DriveData.LinearDriveConstraint.PositionTarget) * DriveTargetTM;

			Constraint->SetLinearDrivePositionTarget(DriveTargetTM.GetTranslation());
			Constraint->SetAngularDrivePositionTarget(DriveTargetTM.GetRotation());

			if (Record.PreviousDriveTargetUpdateCounter + 1 == UpdateCounter && TargetVelocityMultiplier > 0.0f)
			{
				if (DeltaTime > SMALL_NUMBER)
				{
					const FTransform DriveTargetTMDelta =
						Record.PreviousDriveTargetTM.GetRelativeTransformReverse(DriveTargetTM);
					const FVector Velocity = DriveTargetTMDelta.GetTranslation() / DeltaTime;
					const FVector AngularVelocity =
						DriveTargetTMDelta.GetRotation().GetShortestArcWith(
							FQuat::Identity).ToRotationVector() / DeltaTime;
					if (!Velocity.ContainsNaN() && !AngularVelocity.ContainsNaN())
					{
						Constraint->SetLinearDriveVelocityTarget(Velocity * TargetVelocityMultiplier);
						Constraint->SetAngularDriveVelocityTarget(AngularVelocity * TargetVelocityMultiplier);
					}
				}
			}
			else
			{
				Constraint->SetLinearDriveVelocityTarget(Chaos::FVec3(0));
				Constraint->SetAngularDriveVelocityTarget(Chaos::FVec3(0));
			}
			Record.PreviousDriveTargetUpdateCounter = UpdateCounter;
			Record.PreviousDriveTargetTM = DriveTargetTM;
		}
	}
}

//======================================================================================================================
static FTransform CalculateTargetTM(
	const URigHierarchy&               Hierarchy,
	const Chaos::FPBDJointSettings&    JointSettings,
	FRigControlRecord&                 Record)
{
	const FTransform ChildTM = Record.CachedChildBodyComponent.GetComponentKey().IsValid() ?
		GetGlobalTransform(Hierarchy, Record.CachedChildBodyComponent) : FTransform();

	const FTransform ChildTargetTM = JointSettings.ConnectorTransforms[ConstraintChildIndex] * ChildTM;

	if (Record.CachedParentBodyComponent.GetComponentKey().IsValid())
	{
		const FTransform ParentTM(
			GetGlobalTransform(Hierarchy, Record.CachedParentBodyComponent));
		const FTransform ParentTargetTM = JointSettings.ConnectorTransforms[ConstraintParentIndex] * ParentTM;
		return ChildTargetTM.GetRelativeTransform(ParentTargetTM);
	}
	return ChildTargetTM;
}

//======================================================================================================================
void FRigPhysicsSolver::CheckForResetsPrePhysics(
	const URigHierarchy& Hierarchy, FRigPhysicsSolverComponent& SolverComponent, const float DeltaTime)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_CheckForResetsPrePhysics);

	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	double SpeedThresholdForResetSquared = SolverSettings.bResetFromKinematicSpeed ?
		FMath::Square(SolverSettings.KinematicSpeedThresholdForReset) : TNumericLimits<double>::Max();

	double AccelerationThresholdForResetSquared = SolverSettings.bResetFromKinematicAcceleration ?
		FMath::Square(SolverSettings.KinematicAccelerationThresholdForReset) : TNumericLimits<double>::Max();

	if (SpeedThresholdForResetSquared == TNumericLimits<double>::Max() &&
		AccelerationThresholdForResetSquared == TNumericLimits<double>::Max())
	{
		return;
	}

	double HighestSpeedSq = -1.0;
	double HighestAccelerationSq = -1.0;

	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		FRigBodyRecord& Record = BodyRecordPair.Value;

		if (Record.ActorHandle && Record.ActorHandle->GetIsKinematic())
		{
			if (const FRigPhysicsBodyComponent* PhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
			{
				if (PhysicsBodyComponent->BodySolverSettings.bIncludeInChecksForReset)
				{
					const FVector Velocity = Record.SourceComponentSpaceVelocity;
					const FVector Acceleration =
						DeltaTime > SMALL_NUMBER && (UpdateCounter == PreviousUpdateCounter + 1)
						? (Record.SourceComponentSpaceVelocity - Record.PrevSourceComponentSpaceVelocity) / DeltaTime
						: FVector::ZeroVector;

					HighestSpeedSq = FMath::Max(HighestSpeedSq, Velocity.SquaredLength());
					HighestAccelerationSq = FMath::Max(HighestAccelerationSq, Acceleration.SquaredLength());
				}
			}
		}
	}
	if (HighestSpeedSq > SpeedThresholdForResetSquared || HighestAccelerationSq > AccelerationThresholdForResetSquared)
	{
		if (HighestSpeedSq > SpeedThresholdForResetSquared)
		{
			UE_LOGF(LogRigPhysics, Log, "Speed %f triggered reset", FMath::Sqrt(HighestSpeedSq));
		}
		if (HighestAccelerationSq > AccelerationThresholdForResetSquared)
		{
			UE_LOGF(LogRigPhysics, Log, "Acceleration %f triggered reset", FMath::Sqrt(HighestAccelerationSq));
		}
		SolverComponent.TrackInputCounter = FMath::Max(SolverComponent.TrackInputCounter, 3);
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdatePrePhysics(
	const FRigVMExecuteContext& ExecuteContext, 
	const URigHierarchy&        Hierarchy,
	FRigPhysicsSolverComponent& SolverComponent, 
	const float                 DeltaTime)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdatePrePhysics);

	if (!Simulation.IsValid())
	{
		return;
	}

	if (CollisionActorHandle)
	{
		FTransform BodyRelSimSpaceTM = ConvertCollisionSpaceTransformToSimSpace(
			SolverComponent.SolverSettings, FTransform());
		CollisionActorHandle->SetKinematicTarget(BodyRelSimSpaceTM);
	}

	UpdateBodyRecordsPrePhysics(Hierarchy, SolverComponent, DeltaTime);

	if (bNeedToResetPose)
	{
		// Resuming from a non-tracking pass-through. UpdateBodyRecordsPrePhysics has just
		// refreshed the source transforms (and zeroed velocities, since the update counter is
		// stale from the pause), so we can teleport every dynamic actor onto the input
		// animation pose with zero velocity. The TrackInputCounter bump then holds them
		// kinematic for a few frames while drives warm up.
		ResetPoseFromAnimation(SolverComponent, /*bUseSourceVelocity=*/false);
		SolverComponent.TrackInputCounter = FMath::Max(SolverComponent.TrackInputCounter, 3);
		bNeedToResetPose = false;
	}

	CheckForResetsPrePhysics(Hierarchy, SolverComponent, DeltaTime);

	if (SolverComponent.TrackInputCounter > 0)
	{
		UE_LOGF(LogRigPhysics, Log, "Forcing tracking (counter = %d) of input", 
			SolverComponent.TrackInputCounter);
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateBodiesPrePhysics);
		for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
		{
			FRigBodyRecord& Record = BodyRecordPair.Value;
			if (const FRigPhysicsBodyComponent* PhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
			{
				UpdateBodyPrePhysics(ExecuteContext, SolverComponent, Record, *PhysicsBodyComponent);
			}
		}
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateJointsPrePhysics);
		for (TPair<FRigComponentKey, FRigJointRecord>& JointRecordPair : JointRecords)
		{
			FRigJointRecord& JointRecord = JointRecordPair.Value;

			if (const FRigPhysicsJointComponent* JointComponent =
				GetPhysicsJoint(Hierarchy, JointRecord.CachedPhysicsJointComponent))
			{
				UpdateJointPrePhysics(Hierarchy, JointRecord, *JointComponent, DeltaTime);
			}
		}
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateControlsPrePhysics);
		for (TPair<FRigComponentKey, FRigControlRecord>& ControlRecordPair : ControlRecords)
		{
			FRigControlRecord& ControlRecord = ControlRecordPair.Value;

			if (const FRigPhysicsControlComponent* ControlComponent =
				GetPhysicsControl(Hierarchy, ControlRecord.CachedPhysicsControlComponent))
			{
				UpdateControlPrePhysics(ControlRecord, *ControlComponent, SolverComponent, Hierarchy, DeltaTime);
			}
		}
	}

	{
		DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_ApplyForcesPrePhysics);
		for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
		{
			FRigBodyRecord& Record = BodyRecordPair.Value;

			if (const FRigPhysicsBodyComponent* PhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
			{
				ApplyForcesPrePhysics(ExecuteContext, SolverComponent, Record, *PhysicsBodyComponent);
			}

		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::UpdateControlPrePhysics(
	FRigControlRecord&                 ControlRecord,
	const FRigPhysicsControlComponent& ControlComponent,
	const FRigPhysicsSolverComponent&  SolverComponent,
	const URigHierarchy&               Hierarchy,
	const float                        DeltaTime)
{
	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;

	ImmediatePhysics::FJointHandle* JointHandle = ControlRecord.JointHandle;
	if (!JointHandle)
	{
		return;
	}
	Chaos::FPBDJointConstraintHandle* Constraint = JointHandle->GetConstraint();
	if (!Constraint)
	{
		return;
	}

	if (!ControlComponent.ControlData.bEnabled)
	{
		SetPhysicsJointEnabled(*JointHandle, false);
		return;
	}

	float ThisDeltaTime = DeltaTime;

	if (ControlRecord.PreviousTargetUpdateCounter + 1 != UpdateCounter)
	{
		// If we missed some intermediate updates, then we don't want to use the previous
		// positions etc to calculate velocities. This will mean velocity/damping will be
		// incorrect for one frame, but that's probably OK.
		ThisDeltaTime = 0.0f;
	}

	if ((ControlComponent.DirtyFlags & ERigPhysicsControlComponentDirtyFlags::Data) !=
		ERigPhysicsControlComponentDirtyFlags::None)
	{
		Constraint->SetCollisionEnabled(!ControlComponent.ControlData.bDisableCollision);
		Constraint->SetParentInvMassScale(ControlComponent.ControlData.bOnlyControlChildObject ? 0 : 1);
	}

	// Check for early outs. Make sure that we don't apply velocities using the wrong calculation
	// when the strength/damping is increased in the future by not updating the update counter.

	const ImmediatePhysics::FActorHandle* ChildActorHandle = JointHandle->GetActorHandles()[ConstraintChildIndex];
	const ImmediatePhysics::FActorHandle* ParentActorHandle = JointHandle->GetActorHandles()[ConstraintParentIndex];
	if (!ChildActorHandle || !ParentActorHandle)
	{
		return;
	}

	const Chaos::FPBDJointSettings& JointSettings = Constraint->GetSettings();

	// Note that if we don't have any strength, then we don't calculate the targets. 
	if ((ControlComponent.DirtyFlags & ERigPhysicsControlComponentDirtyFlags::Data) !=
		ERigPhysicsControlComponentDirtyFlags::None)
	{
		ControlComponent.DirtyFlags &= ~ERigPhysicsControlComponentDirtyFlags::Data;
		if (!UpdateDriveSpringDamperSettings(
			*JointHandle, JointSettings, ControlComponent.ControlData, ControlComponent.ControlMultiplier))
		{
			ControlRecord.bCachedHasStrengthOrDamping = false;
			return;
		}
		else
		{
			ControlRecord.bCachedHasStrengthOrDamping = true;
		}
	}
	else
	{
		if (!ControlRecord.bCachedHasStrengthOrDamping)
		{
			return;
		}
	}

	// Update the target point on the child

	// TODO Note that if child is kinematic then getting the control point will be a problem
	// as Chaos doesn't like to return the CoM for kinematics. Consider changing Chaos so that the
	// CoM can still be queried. This workaround (of skipping the call) will result in bad behaviour
	// if the child is kinematic and the parent is dynamic. 
	if (!ChildActorHandle->GetIsKinematic())
	{
		// Note - This simply sets the location and doesn't trigger additional work
		Constraint->SetChildConnectorLocation(
			ControlComponent.ControlData.GetControlPoint(ChildActorHandle));
	}

	FTransform TargetTM(
		ControlComponent.ControlTarget.TargetOrientation,
		ControlComponent.ControlTarget.TargetPosition);

	if (ControlComponent.ControlData.bUseSkeletalAnimation)
	{
		const FTransform ComponentSpaceAnimTargetTM = CalculateTargetTM(Hierarchy, JointSettings, ControlRecord);
		const FTransform SimSpaceAnimTargetTM = ConvertComponentSpaceTransformToSimSpace(
			SolverSettings, ComponentSpaceAnimTargetTM);
		TargetTM = TargetTM * SimSpaceAnimTargetTM;
	}
	else
	{
		TargetTM = ConvertComponentSpaceTransformToSimSpace(SolverSettings, TargetTM);
	}

	Constraint->SetLinearDrivePositionTarget(TargetTM.GetTranslation());
	Constraint->SetAngularDrivePositionTarget(TargetTM.GetRotation());

	if ((ThisDeltaTime * ControlComponent.ControlData.LinearTargetVelocityMultiplier) != 0)
	{
		const FVector Velocity = UE::PhysicsControl::CalculateLinearVelocity(
			ControlRecord.PreviousTargetTM.GetTranslation(), TargetTM.GetTranslation(), ThisDeltaTime);
		Constraint->SetLinearDriveVelocityTarget(
			Velocity * ControlComponent.ControlData.LinearTargetVelocityMultiplier);
	}
	else
	{
		Constraint->SetLinearDriveVelocityTarget(Chaos::FVec3(0));
	}

	if ((ThisDeltaTime * ControlComponent.ControlData.AngularTargetVelocityMultiplier) != 0)
	{
		const FQuat Q = TargetTM.GetRotation();
		const FQuat PrevQ = ControlRecord.PreviousTargetTM.GetRotation();
		const FVector AngularVelocity = UE::PhysicsControl::CalculateAngularVelocity(PrevQ, Q, ThisDeltaTime);
		Constraint->SetAngularDriveVelocityTarget(
			AngularVelocity * ControlComponent.ControlData.AngularTargetVelocityMultiplier);
	}
	else
	{
		Constraint->SetAngularDriveVelocityTarget(Chaos::FVec3(0));
	}

	// TODO we don't actually have a shortcut for if the target is clean
	ControlComponent.DirtyFlags &= ~ERigPhysicsControlComponentDirtyFlags::Target;

	ControlRecord.PreviousTargetTM = TargetTM;
	ControlRecord.PreviousTargetUpdateCounter = UpdateCounter;
}

//======================================================================================================================
// Note that we read back into a target bone, which may have been specified explicitly, or will
// otherwise default to the physics element parent.
void FRigPhysicsSolver::UpdatePostPhysics(
	URigHierarchy&              Hierarchy,
	FRigPhysicsSolverComponent& SolverComponent, 
	const float                 Alpha, 
	const float                 DeltaTime,
	const bool                  bUsingFixedStep)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdatePostPhysics);

	// Note that if Alpha <= 0 then we won't actually get here - StepPhysicsSolver will have
	// returned early.
	check(Alpha > 0.0f);

	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;
	bool bGotInvalidSimulationData = false;

	double PositionThresholdForResetSquared = SolverSettings.bResetFromPosition ?
		FMath::Square(SolverSettings.PositionThresholdForReset) : TNumericLimits<double>::Max();
	double HighestPosition = -1.0;

	// Traverse using the sorted keys
	for (const FRigComponentKey& ComponentKey : SortedBodyComponentKeys)
	{
		FRigBodyRecord& Record = *BodyRecords.Find(ComponentKey);

		if (Record.ActorHandle && Record.TargetElementKey.IsValid())
		{
			// Check the simulation output
			FTransform SimSpaceTM = Record.ActorHandle->GetWorldTransform();
			double DistSq = SimSpaceTM.GetTranslation().SquaredLength();
			if (!SimSpaceTM.IsValid() || DistSq > PositionThresholdForResetSquared)
			{
				HighestPosition = FMath::Max(HighestPosition, FMath::Sqrt(DistSq));
				bGotInvalidSimulationData = true;
			}

			// Calculate the target TM even if we're going to reset - it's likely useful for
			// debugging (and this should be rare!) The component stores the raw physics
			// transform. The Alpha * PhysicsBlendWeight blend happens at hierarchy writeback.
			Record.FinalComponentSpaceTM = ConvertSimSpaceTransformToComponentSpace(SolverSettings, SimSpaceTM);

			if (FRigPhysicsBodyComponent* PhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
			{
				PhysicsBodyComponent->ForceAndTorques.Empty();
				Record.PhysicsBlendWeight = PhysicsBodyComponent->BodyData.PhysicsBlendWeight;
				SetPhysicsBodyComponentState(SolverSettings, Record, *PhysicsBodyComponent);
			}
		}
	}

	if (bGotInvalidSimulationData)
	{
		if (HighestPosition > 0)
		{
			UE_LOGF(LogRigPhysics, Log, "Position %f triggered teleport - resetting pose",
				HighestPosition);
		}
		// Avoid cached transforms being used in controls by bumping the update counter. 
		UpdateCounter += 1;
		// Set this to 3 since it gets decremented at the end of the update, and we need it to take
		// effect at the start of the next update.
		SolverComponent.TrackInputCounter = FMath::Max(SolverComponent.TrackInputCounter, 3);

		// If we found something invalid then we force the simulation to be as good as we can make it,
		// and we don't write back to the hierarchy.
		UE_LOGF(LogRigPhysics, Log, "Resetting state to input pose");
		for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
		{
			const FRigComponentKey& ComponentKey = BodyRecordPair.Key;
			FRigBodyRecord& Record = BodyRecordPair.Value;

			if (const FRigPhysicsBodyComponent* PhysicsBodyComponent =
				GetPhysicsBody(Hierarchy, Record.CachedPhysicsBodyComponent))
			{
				if (Record.ActorHandle && !Record.ActorHandle->GetIsKinematic())
				{
					// Get the TM in component space, and then convert it into sim space.
					FRigElementKey SourceKey = PhysicsBodyComponent->BodySolverSettings.SourceBone;
					if (!SourceKey.IsValid())
					{
						SourceKey = ComponentKey.ElementKey;
					}
					if (SourceKey.IsValid())
					{
						const FTransform SourceSimulationSpaceTM =
							ConvertComponentSpaceTransformToSimSpace(SolverSettings, Record.SourceComponentSpaceTM);
						Record.ActorHandle->SetWorldTransform(SourceSimulationSpaceTM);
					}
					Record.ActorHandle->SetLinearVelocity(FVector::ZeroVector);
					Record.ActorHandle->SetAngularVelocity(FVector::ZeroVector);
				}
			}
		}
	}
	else
	{
		// All is good - write the transforms we cached
		for (const FRigComponentKey& ComponentKey : SortedBodyComponentKeys)
		{
			const FRigBodyRecord& Record = *BodyRecords.Find(ComponentKey);
			if (Record.ActorHandle)
			{
				// Kinematic velocities will be incorrect when using a fixed timestep, because of
				// the rewind. Overwrite them here, since we know exactly what they should be, in
				// case they are subsequently made dynamic. This is particularly important when
				// warming up velocities following a reset.
				if (bUsingFixedStep && Record.ActorHandle->GetIsKinematic())
				{
					FVector V = ConvertComponentSpaceVectorToSimSpace(
						SolverSettings, Record.SourceComponentSpaceVelocity);
					FVector W = ConvertComponentSpaceVectorToSimSpace(
						SolverSettings, Record.SourceComponentSpaceAngularVelocity);
					Record.ActorHandle->SetLinearVelocity(V);
					Record.ActorHandle->SetAngularVelocity(W);
				}

				if (Record.TargetElementKey.IsValid())
				{
					// Combine the overall Alpha with the per-body PhysicsBlendWeight.
					// EffectiveAlpha = 0 leaves the bone at the input animation pose (no write).
					// EffectiveAlpha = 1 writes the raw physics transform. Values in between
					// slerp/lerp toward the source bone in component space.
					//
					// TODO this blends in component space, which can cause joint separation. We
					// probably want an option to blend in local (joint) space, perhaps splitting
					// the alpha into orientation and position. The split below into
					// rotation/translation makes that future rework less intrusive.
					//
					// Note that we set bAffectChildren = true (i.e. don't counter-animate
					// children), so attached animation bones follow physics. We rely on
					// SortedBodyComponentKeys being in root-to-leaf order: setting a parent's
					// global drags its descendants along (because we asked for no counter-
					// animation), but the descendants' own writes come later and reassert their
					// targets, so no body is left displaced by an ancestor's write.
					const float EffectiveAlpha = Alpha * Record.PhysicsBlendWeight;
					if (EffectiveAlpha >= 0.999f)
					{
						Hierarchy.SetGlobalTransform(Record.TargetElementKey, Record.FinalComponentSpaceTM, false, true);
					}
					else if (EffectiveAlpha > 0.0f)
					{
						FTransform BlendedTM = Record.FinalComponentSpaceTM;
						const FQuat BlendedQ = FQuat::Slerp(
							Record.SourceComponentSpaceTM.GetRotation(),
							Record.FinalComponentSpaceTM.GetRotation(), EffectiveAlpha);
						const FVector BlendedT = FMath::Lerp(
							Record.SourceComponentSpaceTM.GetTranslation(),
							Record.FinalComponentSpaceTM.GetTranslation(), EffectiveAlpha);
						BlendedTM.SetRotation(BlendedQ);
						BlendedTM.SetTranslation(BlendedT);
						Hierarchy.SetGlobalTransform(Record.TargetElementKey, BlendedTM, false, true);
					}
					// else EffectiveAlpha <= 0 - skip the write entirely so the bone stays at
					// whatever the input animation set it to.
				}
			}
		}
	}
}

//======================================================================================================================
void FRigPhysicsSolver::StepSimulation(
	const UWorld*                World,
	const AActor*                OwningActorPtr,
	const FRigVMExecuteContext&  ExecuteContext,
	URigHierarchy&               Hierarchy,
	FRigPhysicsSolverComponent&  SolverComponent,
	const float                  DeltaTimeOverride,
	const float                  SimulationSpaceDeltaTimeOverride,
	const float                  Alpha,
	const bool                   bTrackVelocitiesDuringPassThrough)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_StepSimulation);

	// Increment the update counter at the start - and always update it so this tells us our "frame number".
	++UpdateCounter;

	float PhysicsDeltaTime = ExecuteContext.GetDeltaTime();
	if (DeltaTimeOverride > 0.0f)
	{
		PhysicsDeltaTime = DeltaTimeOverride;
	}
	else if (DeltaTimeOverride < 0.0f)
	{
		PhysicsDeltaTime = 0.0f;
	}

	float PhysicsSimulationSpaceDeltaTime = PhysicsDeltaTime;
	if (SimulationSpaceDeltaTimeOverride > 0.0f)
	{
		PhysicsSimulationSpaceDeltaTime = SimulationSpaceDeltaTimeOverride;
	}

	// Capture the interval since the previous evaluation before we overwrite the tracker. A value <
	// 0 (on the first frame after instantiation) means "no prior evaluation" and skips the test.
	// Force tracking for a few frames if the interval is large enough to count as a real gap in
	// evaluation - mirrors the existing speed/acceleration reset path in CheckForResetsPrePhysics.
	{
		const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;
		const double CurrentAbsoluteTime = ExecuteContext.GetAbsoluteTime();
		const double PreviousAbsoluteTime = LastEvaluationAbsoluteTime;
		LastEvaluationAbsoluteTime = CurrentAbsoluteTime;

		if (SolverSettings.bResetFromEvaluationInterval && PreviousAbsoluteTime >= 0.0)
		{
			const double Interval = CurrentAbsoluteTime - PreviousAbsoluteTime;
			// Slightly more than the typical variability between absolute delta times and the frame delta time
			double constexpr DeltaTimeMargin = 0.01;
			if (Interval > SolverSettings.EvaluationIntervalThresholdForReset && 
				Interval > (PhysicsDeltaTime + DeltaTimeMargin))
			{
				UE_LOGF(LogRigPhysics, Log, "Evaluation interval %f triggered reset", Interval);
				SolverComponent.TrackInputCounter = FMath::Max(SolverComponent.TrackInputCounter, 3);
			}
		}
	}

	// Lock against access to the simulation, in case of access by the WorldObject task
	UE::TScopeLock<FTransactionallySafeCriticalSection> Lock(*SimulationMutex.Get());

	// Pass-through when Alpha is zero (or negative): bypass the full simulation.
	if (Alpha <= 0.0f)
	{
		if (bTrackVelocitiesDuringPassThrough && PhysicsDeltaTime > 0.0f)
		{
			// Tracking mode: keep source transforms / velocities / sim-space state current and
			// snap the dynamic actor handles onto the animation pose each frame, so that on
			// resume the simulation continues from where the input is (not where the sim was
			// when we entered pass-through). Sim-space update must run before Instantiate so
			// newly-created bodies land in the right place.
			SimulationSpaceData = UpdateSimulationSpaceStateAndCalculateData(
				ExecuteContext, Hierarchy, SolverComponent,
				PhysicsSimulationSpaceDeltaTime, ExecuteContext.GetAbsoluteTime());
			Instantiate(ExecuteContext, Hierarchy, SolverComponent);
			if (Simulation)
			{
				UpdateBodyRecordsPrePhysics(Hierarchy, SolverComponent, PhysicsDeltaTime);
				ResetPoseFromAnimation(SolverComponent, /*bUseSourceVelocity=*/true);
				// Mark this frame as consecutive so the velocity staleness checks in
				// UpdateBodyRecordPrePhysics and UpdateSimulationSpaceStateAndCalculateData
				// continue to produce valid velocities across the pass-through window.
				PreviousUpdateCounter = UpdateCounter;
				bNeedToResetPose = false;
			}
		}
		else if (!bNeedToInstantiate)
		{
			// Non-tracking pass-through on an instantiated sim: schedule a reset for resume.
			// Leaving PreviousUpdateCounter stale means the velocity staleness checks will zero
			// velocities on the first real step.
			bNeedToResetPose = true;
		}

		// Drop queued forces - we're not simulating this frame, so they would otherwise leak
		// across the pause.
		ClearBodyForceAndTorques(Hierarchy);
		return;
	}

	// We need to know about the simulation space etc before we can instantiate anything into the right place
	SimulationSpaceData = UpdateSimulationSpaceStateAndCalculateData(
		ExecuteContext, Hierarchy, SolverComponent, PhysicsSimulationSpaceDeltaTime, ExecuteContext.GetAbsoluteTime());

	// We instantiate when we do the first simulation - this makes sure any changes applied by
	// the user have been made. It also means there is no overhead if physics is never stepped.
	// However, there may be a hitch due to the creation, so it may also happen during construction.
	Instantiate(ExecuteContext, Hierarchy, SolverComponent); // There is an early out if it's already been done

	if (!Simulation)
	{
		return;
	}

	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;
	const FRigPhysicsSimulationSpaceMotion& SpaceMotion = SolverComponent.SpaceMotion;

	float FixedTimeStep = CVarControlRigPhysicsFixedTimeStepOverride.GetValueOnAnyThread() < 0
		? SolverSettings.FixedTimeStep : CVarControlRigPhysicsFixedTimeStepOverride.GetValueOnAnyThread();
	int MaxTimeSteps = CVarControlRigPhysicsMaxTimeStepsOverride.GetValueOnAnyThread() < 0
		? SolverSettings.MaxTimeSteps : CVarControlRigPhysicsMaxTimeStepsOverride.GetValueOnAnyThread();
	float MaxDeltaTime = CVarControlRigPhysicsMaxDeltaTimeOverride.GetValueOnAnyThread() < 0
		? SolverSettings.MaxDeltaTime : CVarControlRigPhysicsMaxDeltaTimeOverride.GetValueOnAnyThread();

	// Set settings that might change
	Simulation->SetSolverSettings(
		FixedTimeStep,
		SolverSettings.CollisionBoundsExpansion,
		SolverSettings.MaxDepenetrationVelocity,
		SolverSettings.bUseLinearJointSolver,
		SolverSettings.PositionIterations,
		SolverSettings.VelocityIterations,
		SolverSettings.ProjectionIterations,
		SolverSettings.bUseManifolds);

	Simulation->SetUseMinStepTime(false);
	Simulation->SetUseFixedStepTolerance(false);

	Chaos::FCollisionDetectorSettings CollisionDetectorSettings = Simulation->GetCollisionDetectorSettings();
	CollisionDetectorSettings.BoundsVelocityInflation = SolverSettings.BoundsVelocityMultiplier;
	CollisionDetectorSettings.MaxVelocityBoundsExpansion = SolverSettings.MaxVelocityBoundsExpansion;
	CollisionDetectorSettings.bAllowCCD = SolverSettings.bAllowCCD;
	Simulation->SetCollisionDetectorSettings(CollisionDetectorSettings);

	// This gets reset to 100 after every simulation step!
	Simulation->SetMaxNumRollingAverageStepTimes(SolverSettings.MaxNumRollingAverageStepTimes);

	// Other settings - would normally be static (so TODO move this)
	ChaosJointSolverSettings.bSolvePositionLast = SolverSettings.bSolveJointPositionsLast;
	ChaosJointSolverSettings.bSortEnabled = true;

	// Simulation space
	Chaos::FSimulationSpaceSettings ChaosSimulationSpaceSettings = Simulation->GetSimulationSpaceSettings();
	ChaosSimulationSpaceSettings.bEnabled = (SpaceMotion.InertialForces.Amount > 0.0f);
	ChaosSimulationSpaceSettings.ExternalLinearEtherDrag = SpaceMotion.Drag.ExternalLinearDrag;
	ChaosSimulationSpaceSettings.LinearVelocityAlpha = SpaceMotion.Drag.LinearDragMultiplier;
	ChaosSimulationSpaceSettings.AngularVelocityAlpha = SpaceMotion.Drag.AngularDragMultiplier;
	ChaosSimulationSpaceSettings.LinearAccelerationAlpha = SpaceMotion.InertialForces.LinearEulerAmount;
	ChaosSimulationSpaceSettings.EulerAlpha = SpaceMotion.InertialForces.AngularEulerAmount;
	ChaosSimulationSpaceSettings.CentrifugalAlpha = SpaceMotion.InertialForces.CentrifugalAmount;
	ChaosSimulationSpaceSettings.CoriolisAlpha = SpaceMotion.InertialForces.CoriolisAmount;
	Simulation->SetSimulationSpaceSettings(ChaosSimulationSpaceSettings);
	Simulation->UpdateSimulationSpace(
		SimulationSpaceState.SimulationSpaceTM,
		SpaceMotion.InertialForces.Amount * SimulationSpaceData.LinearVelocity,
		SpaceMotion.InertialForces.Amount * SimulationSpaceData.AngularVelocity,
		SpaceMotion.InertialForces.Amount * SimulationSpaceData.LinearAcceleration,
		SpaceMotion.InertialForces.Amount * SimulationSpaceData.AngularAcceleration);

	UpdateWorldObjectsPrePhysics(SolverSettings);

	// Only update if there is a delta time:
	// * We don't want to update our previous TMs and store the dt - because that would end up
	//   implying infinite velocities
	// * We don't to update kinematic bodies with the new TMs because, since the simulated ones
	//   won't move, that would break the pose.
	// * We can't actually simulate with dt = 0
	if (PhysicsDeltaTime > 0.0f)
	{
		UpdatePrePhysics(ExecuteContext, Hierarchy, SolverComponent, PhysicsDeltaTime);

		Simulation->Simulate(
			PhysicsDeltaTime, MaxDeltaTime, MaxTimeSteps,
			SimulationSpaceData.Gravity, &ChaosJointSolverSettings);

		PreviousUpdateCounter = UpdateCounter;
	}

	// Always do a read-back, even for zero Dt
	UpdatePostPhysics(Hierarchy, SolverComponent, FMath::Clamp(Alpha, 0.0f, 1.0f), PhysicsDeltaTime, FixedTimeStep > 0);

	// Trigger the update of world objects as soon as possible. Note that this needs to be done
	// after the last access to the simulation, because the game-thread task will start
	// adding/removing objects from our simulation, and we can't let that work overlap with what we
	// do here.

	UpdateWorldObjectsPostPhysics(World, SolverSettings, OwningActorPtr);

	if (SolverComponent.TrackInputCounter > 0)
	{
		--SolverComponent.TrackInputCounter;
	}
}
