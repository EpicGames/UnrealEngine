// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsData.h"

#include "PhysicsControlPoseData.h"

#include "Rigs/RigHierarchyComponents.h"
#include "Rigs/RigHierarchyCache.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsDeclares.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "Chaos/PBDJointConstraintTypes.h"
#include "Misc/TransactionallySafeCriticalSection.h"

#include "RigPhysicsSolver.generated.h"

class UControlRig;

struct FRigVMDrawInterface;
struct FRigVMExecuteContext;
struct FRigPhysicsSolverComponent;

struct FRigPhysicsCollision;
struct FRigPhysicsDynamics;
struct FRigPhysicsBodyComponent;
struct FRigPhysicsControlComponent;
struct FRigPhysicsJointComponent;

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
DECLARE_LOG_CATEGORY_EXTERN(LogRigPhysics, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogRigPhysics, Log, All);
#endif

//======================================================================================================================
// For internal use only
//======================================================================================================================
struct FRigPhysicsIgnorePair
{
	FRigPhysicsIgnorePair(const FRigComponentKey& InA, const FRigComponentKey& InB)
		: A(InA), B(InB) {}
	FRigComponentKey A;
	FRigComponentKey B;

	bool operator==(const FRigPhysicsIgnorePair& Other) const
	{
		return (A == Other.A && B == Other.B) || (A == Other.B && B == Other.A);
	}
};

FORCEINLINE uint32 GetTypeHash(const FRigPhysicsIgnorePair& Pair)
{
	return HashCombine(GetTypeHash(FMath::Min(Pair.A, Pair.B)), GetTypeHash(FMath::Max(Pair.A, Pair.B)));
}
typedef TSet<FRigPhysicsIgnorePair> FRigPhysicsIgnorePairs;

//======================================================================================================================
// For internal use only, to keep track of world objects
//======================================================================================================================
struct FWorldObjectKey
{
	FWorldObjectKey(uint32 InComponentId, int32 InInstanceIndex)
		: ComponentId(InComponentId), InstanceIndex(InInstanceIndex) {}

	uint32 ComponentId;
	int32  InstanceIndex;

	bool operator==(const FWorldObjectKey& Other) const
	{
		return ComponentId == Other.ComponentId && InstanceIndex == Other.InstanceIndex;
	}
};

FORCEINLINE uint32 GetTypeHash(const FWorldObjectKey& Key)
{
	return HashCombine(Key.ComponentId, static_cast<uint32>(Key.InstanceIndex));
}

//======================================================================================================================
// For internal use only, to keep track of objects we have created
//======================================================================================================================
struct FRigBodyRecord
{
	// Things that are set during instantiation
	ImmediatePhysics::FActorHandle* ActorHandle = nullptr;

	// Cached component pointer so the per-tick solver loops can skip the hierarchy TMap lookup
	// that Hierarchy.FindComponent(Key) performs. Auto-invalidated by the topology hash check
	// inside FCachedRigComponent::UpdateCache.
	FCachedRigComponent CachedPhysicsBodyComponent;

	// Cache the element key for where we will write the simulation result
	FRigElementKey TargetElementKey;

	// The final/simulated transform is stored before writing it into the output, so we can avoid
	// corrupting the output if anything is bad and we need to reset.
	FTransform FinalComponentSpaceTM;

	// These source (i.e. bone/element) transforms are updated for all records in UpdatePrePhysics.
	// The times/validity are determined by the CurrentDeltaTime, PrevDeltaTime and update counters
	// in the simulation itself.
	FTransform SourceComponentSpaceTM;
	FVector    SourceComponentSpaceVelocity = FVector::ZeroVector;
	FVector    SourceComponentSpaceAngularVelocity = FVector::ZeroVector;

	FTransform PrevSourceComponentSpaceTM;
	FVector    PrevSourceComponentSpaceVelocity = FVector::ZeroVector;
	FVector    PrevSourceComponentSpaceAngularVelocity = FVector::ZeroVector;

	// Cached from PhysicsBodyComponent.BodyData.PhysicsBlendWeight inside UpdatePostPhysics
	// so the hierarchy-writeback loop can compute EffectiveAlpha = Alpha * PhysicsBlendWeight
	// without a second component lookup. Default 1.0 = fully physics.
	float PhysicsBlendWeight = 1.0f;
};

//======================================================================================================================
// For internal use only, to keep track of joints we have created
//======================================================================================================================
struct FRigJointRecord
{
	// Things that are set during instantiation
	ImmediatePhysics::FJointHandle* JointHandle = nullptr;

	// Cached joint component pointer (see FRigBodyRecord for rationale).
	FCachedRigComponent             CachedPhysicsJointComponent;

	// Cached parent/child body components. These keys are filled in when the record is created,
	// even if the original key is set to pick up the components automatically. Cached so the
	// per-tick joint update can skip both the component TMap lookup and (via GetGlobalTransform
	// on the cache) the element key TMap lookup.
	FCachedRigComponent             CachedParentBodyComponent;
	FCachedRigComponent             CachedChildBodyComponent;

	// Things that are updated as the simulation progresses

	// The drive works with velocities so we store the previous target transform, and when it was stored.
	FTransform                      PreviousDriveTargetTM;
	// This is stored from the main solver update counter, marking when the previous drive TM was valid.
	int64                           PreviousDriveTargetUpdateCounter = -999;
};

//======================================================================================================================
// For internal use only, to keep track of controls we have created
//======================================================================================================================
struct FRigControlRecord
{
	// Things that are set during instantiation
	ImmediatePhysics::FJointHandle* JointHandle = nullptr;

	// Cached control component pointer (see FRigBodyRecord for rationale).
	FCachedRigComponent             CachedPhysicsControlComponent;

	// Cached parent/child body components (see FRigJointRecord for rationale). An invalid
	// parent here indicates a sim-space control attached to the simulation actor.
	FCachedRigComponent             CachedParentBodyComponent;
	FCachedRigComponent             CachedChildBodyComponent;

	// Things that are updated as the simulation progresses

	// The control works with velocities so we store the previous target, and when it was stored.
	FTransform                      PreviousTargetTM;
	// This is stored from the main solver update counter, marking when the previous target TM was valid.
	int64                           PreviousTargetUpdateCounter = -999;

	// When the FRigPhysicsControlComponent is dirty, this is calculated from the strengths etc, and
	// then cached. Then if FRigPhysicsControlComponent is clean, we can see whether there is any
	// strength/damping that would require the target to be updated.
	bool bCachedHasStrengthOrDamping = true;
};

//======================================================================================================================
// True if any per-element draw would actually emit geometry, given the rig's bShow* flags AND the
// per-element ControlRig.Physics.Show*Override CVars
//======================================================================================================================
bool RigPhysicsShouldVisualize(const FRigPhysicsVisualizationSettings1& VisualizationSettings);

//======================================================================================================================
// This represents the low level simulation, plus all the objects and controls we make to go in it
//======================================================================================================================
USTRUCT()
struct FRigPhysicsSolver
{
public:
	GENERATED_BODY()

	FRigPhysicsSolver() {}
	FRigPhysicsSolver(const FRigComponentKey& SolverComponentKey);

	// This will initialise/create the simulation and then create everything we need in it. 
	void Instantiate(
		const FRigVMExecuteContext&       ExecuteContext, 
		URigHierarchy&                    Hierarchy, 
		const FRigPhysicsSolverComponent& SolverComponent);

	// Integrates the simulation forwards.
	// If DeltaTimeOverride is +ve, then that value is used.
	// If it is zero, then delta time is taken from the execute context
	// If it is negative, then the simulation isn't stepped.
	// When Alpha is <= 0 the simulation is bypassed entirely (pass-through). If
	// bTrackVelocitiesDuringPassThrough is true the solver still updates source bone transforms
	// and sim-space state each frame so velocities are valid on resume. Otherwise the resume
	// schedules a brief kinematic warm-up via TrackInputCounter.
	void StepSimulation(
		const UWorld*                World,
		const AActor*                OwningActorPtr,
		const FRigVMExecuteContext&  ExecuteContext,
		URigHierarchy&               Hierarchy,
		FRigPhysicsSolverComponent&  SolverComponent,
		const float                  DeltaTimeOverride,
		const float                  SimulationSpaceDeltaTimeOverride,
		const float                  Alpha,
		const bool                   bTrackVelocitiesDuringPassThrough);

	// Draws shapes etc AND (potentially) enables the low-level Chaos debug draw
	void Draw(
		FRigVMDrawInterface*                     DI,
		const FRigPhysicsSolverSettings&         SolverSettings,
		const FRigPhysicsVisualizationSettings1& VisualizationSettings,
		const UWorld*                            DebugWorld) const;

	// Represents the properties of the simulation space - calculated near the beginning of the update.
	// Note that all these are specified in the simulation space itself
	struct FSimulationSpaceData
	{
		FVector LinearVelocity;
		// Angular velocity is in rad/s
		FVector AngularVelocity;
		FVector LinearAcceleration;
		// Angular acceleration is in rad/s/s
		FVector AngularAcceleration;
		FVector Gravity;
	};

	// Returns the simulation space data, as calculated at the start of the last step
	const FSimulationSpaceData& GetSimulationSpaceData() const { return SimulationSpaceData; }

	// Returns the component that we're associated with/owned by
	const FRigComponentKey& GetSolverComponentKey() const { return PhysicsSolverComponentKey; }

private:
	// Used by the world-space to simulation-space motion transfer system in Component- or
	// Bone-Space sims, and preserved between updates.
	struct FSimulationSpaceState
	{
		FTransform ComponentTM;
		FTransform BoneRelComponentTM;

		// The world transform of the simulation space
		FTransform SimulationSpaceTM;
		FTransform PrevSimulationSpaceTM;
		FTransform PrevPrevSimulationSpaceTM;
		// The time between SimulationSpaceTM and PrevSimulationSpaceTM
		float      Dt = 1.0f; 
		// The time between PrevSimulationSpaceTM and PrevPrevSimulationSpaceTM
		float      PrevDt = 1.0f;
	};

	// Creates the low level simulation
	void InitialiseSimulation();

	// This collects all the bodies associated with the solver and makes records for them
	void InitialiseBodyRecords(const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent);

	// This collects all the joints associated with the solver and makes records for them
	void InitialiseJointRecords(const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent);

	// This collects all the controls associated with the solver and makes records for them
	void InitialiseControlRecords(const URigHierarchy& Hierarchy, const FRigPhysicsSolverComponent& SolverComponent);

	// Creates low level bodies
	void InstantiatePhysicsBodies(
		URigHierarchy&                    Hierarchy,
		const FRigPhysicsSolverComponent& SolverComponent,
		FRigPhysicsIgnorePairs&           IgnorePairs);

	// Creates low level physics joints 
	void InstantiatePhysicsJoints(
		const URigHierarchy&              Hierarchy,
		const FRigPhysicsSolverComponent& SolverComponent,
		FRigPhysicsIgnorePairs&           IgnorePairs);

	// Creates low level controls
	void InstantiateControls(
		const URigHierarchy&              Hierarchy,
		const FRigPhysicsSolverComponent& SolverComponent,
		FRigPhysicsIgnorePairs&           IgnorePairs);

	// Creates collision shapes associated with the solver. Also applies IgnorePairs
	void InstantiateSolverCollision(
		const FRigPhysicsSolverComponent& SolverComponent,
		FRigPhysicsIgnorePairs&           IgnorePairs);

	// Returns true if the component is physics, and its solver matches the/our solver component
	// (directly, or automatically)
	bool ShouldComponentBeInSimulation(
		const URigHierarchy&              Hierarchy,
		const FRigPhysicsSolverComponent& SolverComponent,
		const FRigComponentKey&           ComponentKey) const;

	// Walks the parent-element chain starting from InChildElementKey's first parent and returns
	// the first component on any ancestor that ShouldComponentBeInSimulation accepts. Returns an
	// invalid key if no ancestor has a qualifying component. Used to auto-resolve the parent body
	// for joint and control components when the user left the parent key blank - physics assets
	// may skip bodies on intermediate joints, so we keep walking until we find one.
	FRigComponentKey FindAncestorBodyInSimulation(
		const URigHierarchy&              Hierarchy,
		const FRigPhysicsSolverComponent& SolverComponent,
		const FRigElementKey&             InChildElementKey) const;

	// Creates an actor with collision. This will be dynamic if Dynamics is valid, or otherwise kinematic
	ImmediatePhysics::FActorHandle* CreateBody(
		const FName                        BodyName,
		const FRigPhysicsCollision&        Collision,
		const FRigPhysicsDynamics*         Dynamics,
		const FPhysicsControlModifierData* BodyData,
		const FTransform&                  BodyRelSimSpaceTM) const;

	// This releases the actors etc in the simulation, destroys the simulation, and resets our
	// internal records.
	void DestroyPhysicsSimulation();

	void UpdatePrePhysics(
		const FRigVMExecuteContext&        ExecuteContext, 
		const URigHierarchy&               Hierarchy,
		FRigPhysicsSolverComponent&        SolverComponent, 
		const float                        DeltaTime);

	void UpdateBodyRecordPrePhysics(
		const URigHierarchy&               Hierarchy,
		const FRigPhysicsSolverComponent&  SolverComponent,
		const float                        DeltaTime,
		FRigBodyRecord&                    Record,
		const FRigPhysicsBodyComponent*    PhysicsComponent);

	// Iterates all BodyRecords and calls UpdateBodyRecordPrePhysics on each. Used by
	// UpdatePrePhysics and by the Alpha=0 pass-through path when velocity tracking is enabled.
	void UpdateBodyRecordsPrePhysics(
		const URigHierarchy&               Hierarchy,
		const FRigPhysicsSolverComponent&  SolverComponent,
		const float                        DeltaTime);

	// Drops all queued ForceAndTorques entries across all bodies. Called on pass-through frames
	// where ApplyForcesPrePhysics/UpdatePostPhysics won't run, so the queue cannot leak across
	// the pause.
	void ClearBodyForceAndTorques(URigHierarchy& Hierarchy);

	// Snaps every dynamic actor handle to the current Record.SourceComponentSpaceTM (after
	// converting into simulation space). Used by the Alpha=0 pass-through path so that during
	// (and on resume from) pass-through, the simulation is at the input animation pose rather
	// than the last simulated pose. When bUseSourceVelocity is true, the actor velocities are
	// set from Record.SourceComponent[Angular]Velocity (tracking mode - smooth resume).
	// Otherwise they are zeroed (reset mode).
	void ResetPoseFromAnimation(
		const FRigPhysicsSolverComponent& SolverComponent,
		const bool                        bUseSourceVelocity);

	void UpdateBodyPrePhysics(
		const FRigVMExecuteContext&        ExecuteContext, 
		const FRigPhysicsSolverComponent&  SolverComponent,
		const FRigBodyRecord&              Record, 
		const FRigPhysicsBodyComponent&    PhysicsComponent);

	void ApplyForcesPrePhysics(
		const FRigVMExecuteContext&        ExecuteContext, 
		const FRigPhysicsSolverComponent&  SolverComponent,
		const FRigBodyRecord&              Record, 
		const FRigPhysicsBodyComponent&    PhysicsComponent);

	void UpdateJointPrePhysics(
		const URigHierarchy&               Hierarchy, 
		FRigJointRecord&                   JointRecord,
		const FRigPhysicsJointComponent&   JointComponent, 
		const float                        DeltaTime);

	void UpdateControlPrePhysics(
		FRigControlRecord&                 ControlRecord,
		const FRigPhysicsControlComponent& ControlComponent,
		const FRigPhysicsSolverComponent&  SolverComponent,
		const URigHierarchy&               Hierarchy,
		const float                        DeltaTime);

	void UpdatePostPhysics(
		URigHierarchy&              Hierarchy,
		FRigPhysicsSolverComponent& SolverComponent, 
		const float                 Alpha, 
		const float                 DeltaTime,
		const bool                  bUsingFixedStep);

	// This will walk through the WorldObjects, which should now be updated, and update the simulation
	void UpdateWorldObjectsPrePhysics(const FRigPhysicsSolverSettings& SolverSettings);

	// This will trigger a game-thread update of WorldObjects, ready for the next update
	void UpdateWorldObjectsPostPhysics(
		const UWorld*                    World,
		const FRigPhysicsSolverSettings& SolverSetting, 
		const AActor*                    OwningActorPtr);

	void InitSimulationSpace(const FTransform& ComponentToWorld, const FTransform& BoneToComponent);

	void CheckForResetsPrePhysics(
		const URigHierarchy& Hierarchy, FRigPhysicsSolverComponent& SolverComponent, const float DeltaTime);

	// AbsoluteTime is used for calculating turbulence when applying the "wind"
	FSimulationSpaceData UpdateSimulationSpaceStateAndCalculateData(
		const FRigVMExecuteContext&         ExecuteContext,
		const URigHierarchy&                Hierarchy,
		const FRigPhysicsSolverComponent&   SolverComponent,
		const float                         Dt,
		const double                        AbsoluteTime);

	static FTransform GetSpaceTransform(
		ERigPhysicsSimulationSpace Space, const FTransform& ComponentTM, const FTransform& BoneTM);

	// Returns the simulation space transform, in world space
	FTransform GetSimulationSpaceTransform(const FRigPhysicsSolverSettings& SolverSettings) const;

	// Converts a transform from component space (e.g. coming from the owning control rig) into the
	// simulation space
	FTransform ConvertComponentSpaceTransformToSimSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FTransform& ComponentSpaceTM) const;

	// Converts a component space position to simulation space, including translation
	FVector ConvertComponentSpacePositionToSimSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& ComponentSpacePosition) const;

	// Converts a component space vector to simulation space, ignoring translation
	FVector ConvertComponentSpaceVectorToSimSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& ComponentSpaceVector) const;

	// Converts a transform specified in world space to the simulation space
	FTransform ConvertWorldTransformToSimSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FTransform& WorldSpaceTM) const;

	// Converts a position specified in world space to the simulation space
	FVector ConvertWorldPositionToSimSpaceNoScale(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& WorldSpacePosition) const;

	// Converts a vector specified in world space to the simulation space (e.g. converting gravity)
	FVector ConvertWorldVectorToSimSpaceNoScale(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& WorldSpaceVector) const;

	// Converts a transform from the simulation space to component space (e.g. for writing back to
	// the owning control rig)
	FTransform ConvertSimSpaceTransformToComponentSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FTransform& SimSpaceTM) const;

	// Converts a simulation space position to component space, including translation
	FVector ConvertSimSpacePositionToComponentSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& SimSpacePosition) const;

	// Converts a simulation space vector to component space (ignoring translation)
	FVector ConvertSimSpaceVectorToComponentSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FVector& SimSpaceVector) const;

	// Converts a collision space transform to simulation space
	FTransform ConvertCollisionSpaceTransformToSimSpace(
		const FRigPhysicsSolverSettings& SolverSettings, const FTransform& TM) const;

	// Gets the simulation actor handle for a component key. Note that the component key could be a
	// body or a solver component. Can return nullptr
	ImmediatePhysics::FActorHandle* GetActor(const FRigComponentKey& ComponentKey) const;

	// Updates the PhysicsBodyComponent with the state of the rigid body from the record. Assumes
	// that FinalComponentSpaceTM has been set.
	void SetPhysicsBodyComponentState(
		const FRigPhysicsSolverSettings& SolverSettings, 
		FRigBodyRecord&                  Record, 
		FRigPhysicsBodyComponent&        PhysicsBodyComponent) const;

private:
	FRigComponentKey        PhysicsSolverComponentKey;

	// All the bodies, but in no particular order
	TMap<FRigComponentKey, FRigBodyRecord> BodyRecords;

	// Ordering so that we can traverse from root to leaf bones
	TArray<FRigComponentKey> SortedBodyComponentKeys;

	// All the joints
	TMap<FRigComponentKey, FRigJointRecord> JointRecords;

	// All the controls
	TMap<FRigComponentKey, FRigControlRecord> ControlRecords;

	TSharedPtr<ImmediatePhysics::FSimulation> Simulation;

	// Used to store things the simulation collision shape. May be offset from the origin if
	// collision is in a different space to the simulation.
	ImmediatePhysics::FActorHandle* CollisionActorHandle = nullptr;

	// Used to make controls when they're not attached to another simulated body. Will always be at
	// the origin.
	ImmediatePhysics::FActorHandle* SimulationActorHandle = nullptr;

	Chaos::FPBDJointSolverSettings ChaosJointSolverSettings;

	FSimulationSpaceState SimulationSpaceState;

	// Retain the data - we don't actually need to but (a) it makes it available for debugging and
	// (b) it avoids passing it through the functions.
	FSimulationSpaceData SimulationSpaceData;

	// This is incremented at the very beginning of each simulation step, used to identify when previously
	// calculated values are valid. Note that the universe will roll over before a int64 does
	int64 UpdateCounter = 0;

	// The update counter when PrevSourceComponentSpaceTM etc were written. Check this before using them
	int64 PreviousUpdateCounter = -999;

	// This will be set to false after instantiation, which can happen manually during construction,
	// or be postponed until there is a simulation step.
	bool bNeedToInstantiate = true;

	// Set true when StepSimulation ran in non-tracking pass-through mode (Alpha <= 0,
	// bTrackVelocitiesDuringPassThrough false) on an already-instantiated sim. Consumed at the
	// top of the next non-pass-through step by bumping SolverComponent.TrackInputCounter so all
	// dynamic bodies warm up over a few frames.
	bool bNeedToResetPose = false;

	// Absolute time (from the rig's ExecuteContext) at the start of the most recent StepSimulation
	// call. A value < 0 means "no prior evaluation" - the interval test is skipped on the first
	// frame after instantiation. Populated regardless of bResetFromEvaluationInterval so toggling
	// the flag on doesn't have a one-frame priming delay.
	double LastEvaluationAbsoluteTime = -1.0;

	// Record of a world object that we want to track, and "import" into our simulation for collision.
	struct FWorldObject
	{
		FWorldObject() : ActorHandle(nullptr), LastSeenUpdateCounter(0) {}
		FWorldObject(ImmediatePhysics::FActorHandle* InActorHandle, int64 InLastSeenUpdateCounter)
			: ActorHandle(InActorHandle)
			, LastSeenUpdateCounter(InLastSeenUpdateCounter) {
		}

		// The object we are tracking
		TWeakObjectPtr<UPrimitiveComponent> WorldPrimitiveComponent;

		// The actor in our simulation
		ImmediatePhysics::FActorHandle* ActorHandle;

		// The transform of the world object. This will be updated in the game-thread task.
		FTransform ComponentWorldTransform;

		// When the world object was last seen (i.e. captured by an overlap), used to detect when we
		// should expire/forget about it.
		int64 LastSeenUpdateCounter;

		// For instanced static mesh components, the index of the specific instance this actor
		// represents. INDEX_NONE for non-instanced components.
		int32 InstanceBodyIndex = INDEX_NONE;

		// This returns true if the object hasn't been seen recently, or is invalid
		bool GetExpired(int64 InUpdateCounter) const
		{
			if (LastSeenUpdateCounter != -1 && InUpdateCounter > LastSeenUpdateCounter + 1)
			{
				return true;
			}
			if (!WorldPrimitiveComponent.IsValid() || !ActorHandle)
			{
				return true;
			}
			return false;
		}
	};

	// The world objects we track for collision. Index uses a composite key composed of the 
	// PrimitiveComponent Id and, possibly, the overlap item index.
	// Note that this is only guaranteed to be unique whilst the object exists, so it is possible
	// (but hopefully unlikely) that an object will be replaced and have the same ID. This will be
	// rectified on the next update, as the record's pointer to the primitive component will be invalid.
	TSharedPtr<TMap<FWorldObjectKey, FWorldObject>> WorldObjects;

	// The box used for overlap tests to find the world objects
	FBox WorldOverlapBox;

	// Guard against access to the low level simulation. Note that this access may come from an
	// async/spawned task, so the mutex itself needs to be shared, as we (as primary owners) may get
	// deleted whilst the spawned task is in flight.
	TSharedPtr<FTransactionallySafeCriticalSection> SimulationMutex;
};

