// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolverComponent.h"
#include "PhysicsControlObjectVersion.h"
#include "RigPhysicsObjectVersion.h"
#include "RigPhysicsLegacyConversion.h"
#include "RigPhysicsSolver.h"

#include "Rigs/RigHierarchy.h"
#if WITH_EDITOR
#include "ControlRigPhysicsEditorStyle.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsSolverComponent)

//======================================================================================================================
static void SerializeSolverComponent(FArchive& Ar, FRigPhysicsSolverComponent& Self)
{
	Ar << Self.SolverSettings;
	if (Ar.CustomVer(FRigPhysicsObjectVersion::GUID) >= FRigPhysicsObjectVersion::SimulationSpaceRegrouping)
	{
		Ar << Self.SpaceMotion;
		Ar << Self.TeleportDetection;
	}
	else if (Ar.IsLoading())
	{
		// Pre-SimulationSpaceRegrouping: a single flat FRigPhysicsSimulationSpaceSettings record
		// followed. Translate it into the new layout (preserving values).
		TranslateLegacyPhysicsSimulationSpaceSettings(Ar, Self.SpaceMotion, Self.TeleportDetection);
	}
}

//======================================================================================================================
void FRigPhysicsSolverComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FPhysicsControlObjectVersion::GUID);
	Ar.UsingCustomVersion(FRigPhysicsObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	SerializeSolverComponent(Ar, *this);
}

//======================================================================================================================
void FRigPhysicsSolverComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	SerializeSolverComponent(Ar, *this);
}

//======================================================================================================================
void FRigPhysicsSolverComponent::OnAddedToHierarchy(URigHierarchy* InHierarchy, URigHierarchyController* InController)
{
	if (!IsProcedural())
	{
		// Default the material here to have friction and restitution. Then the interactions are
		// easily adjusted on the dynamic bodies.
		SolverSettings.Collision.Material.Friction = 1.0f;
		SolverSettings.Collision.Material.Restitution = 1.0f;

		InHierarchy->Notify(ERigHierarchyNotification::ComponentContentChanged, this);
	}
}

//======================================================================================================================
FRigPhysicsSolver* FRigPhysicsSolverComponent::GetPhysicsSolver() const
{
	if (!PhysicsSolverPtr.IsValid())
	{
		const FRigComponentKey SolverComponentKey = GetKey();
		PhysicsSolverPtr = MakeShared<FRigPhysicsSolver>(SolverComponentKey);
	}
	FRigPhysicsSolver* Result = PhysicsSolverPtr.Get();
	check(Result->GetSolverComponentKey() == GetKey());
	return Result;
}

//======================================================================================================================
#if WITH_EDITOR
const FSlateIcon& FRigPhysicsSolverComponent::GetIconForUI() const
{
	static const FSlateIcon SolverIcon = FSlateIcon(
		FControlRigPhysicsEditorStyle::Get().GetStyleSetName(), "ControlRigPhysics.Component.Solver");
	return SolverIcon;
}
#endif

