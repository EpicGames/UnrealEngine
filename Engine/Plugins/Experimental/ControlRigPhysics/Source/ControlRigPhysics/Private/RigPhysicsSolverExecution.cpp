// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolverExecution.h"

#include "RigPhysicsSolver.h"
#include "RigPhysicsLegacyConversion.h"
#include "ControlRigPhysicsModule.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"
#include "ControlRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigPhysicsSolverExecution)

TAutoConsoleVariable<int32> CVarControlRigPhysicsAllowVisualization(
	TEXT("ControlRig.Physics.AllowVisualization"),
#if WITH_EDITOR
	1,
#elif UE_BUILD_SHIPPING || UE_BUILD_TEST
	0,
#else
	1,
#endif
	TEXT("Master allow-switch for control rig physics visualization. Set to 0 to suppress all debug drawing globally (e.g. for profiling). Defaults to 1 in editor and dev builds, 0 in shipping/test."));

TAutoConsoleVariable<int> CVarControlRigPhysicsEnableStepSolver(
	TEXT("ControlRig.Physics.EnableStepSolver"), 1,
	TEXT("Setting to zero disables stepping the solver"));


//======================================================================================================================
FRigUnit_AddPhysicsSolver_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}

	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AddPhysicsSolver can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		PhysicsSolverComponentKey = Controller->AddComponent(
			FRigPhysicsSolverComponent::StaticStruct(), TEXT("PhysicsSolver"), Owner);
		if (PhysicsSolverComponentKey.IsValid())
		{
			if (FRigPhysicsSolverComponent* Component = Cast<FRigPhysicsSolverComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
			{
				Component->SolverSettings = SolverSettings;
				ConvertLegacyPhysicsSimulationSpaceSettings(
					SimulationSpaceSettings, Component->SpaceMotion, Component->TeleportDetection);
			}
		}
	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_AddPhysicsSolver::GetUpgradeInfo() const
{
	FRigUnit_SpawnPhysicsSolver NewNode;
	NewNode.Owner = Owner;
	NewNode.SolverSettings = SolverSettings;
	ConvertLegacyPhysicsSimulationSpaceSettings(
		SimulationSpaceSettings, NewNode.SpaceMotion, NewNode.TeleportDetection);

	FRigVMStructUpgradeInfo Info(*this, NewNode);

	// Sub-pin connections from the old monolithic SimulationSpaceSettings pin need to follow into
	// the new SpaceMotion / TeleportDetection / nested Drag / nested InertialForces sub-pins.
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.SpaceMovementAmount"),                    TEXT("SpaceMotion.InertialForces.Amount"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.VelocityScaleZ"),                         TEXT("SpaceMotion.VerticalMotionScale"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampLinearVelocity"),                   TEXT("SpaceMotion.bClampLinearVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxLinearVelocity"),                      TEXT("SpaceMotion.MaxLinearVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampAngularVelocity"),                  TEXT("SpaceMotion.bClampAngularVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxAngularVelocity"),                     TEXT("SpaceMotion.MaxAngularVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampLinearAcceleration"),               TEXT("SpaceMotion.bClampLinearAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxLinearAcceleration"),                  TEXT("SpaceMotion.MaxLinearAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampAngularAcceleration"),              TEXT("SpaceMotion.bClampAngularAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxAngularAcceleration"),                 TEXT("SpaceMotion.MaxAngularAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.LinearDragMultiplier"),                   TEXT("SpaceMotion.Drag.LinearDragMultiplier"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.AngularDragMultiplier"),                  TEXT("SpaceMotion.Drag.AngularDragMultiplier"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.ExternalLinearDrag"),                     TEXT("SpaceMotion.Drag.ExternalLinearDrag"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.ExternalLinearVelocity"),                 TEXT("SpaceMotion.Drag.ExternalLinearVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.ExternalAngularVelocity"),                TEXT("SpaceMotion.Drag.ExternalAngularVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.ExternalTurbulenceVelocity"),             TEXT("SpaceMotion.Drag.ExternalTurbulenceVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.LinearAccelerationThresholdForTeleport"),  TEXT("TeleportDetection.LinearAccelerationThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.AngularAccelerationThresholdForTeleport"), TEXT("TeleportDetection.AngularAccelerationThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.PositionChangeThresholdForTeleport"),      TEXT("TeleportDetection.PositionChangeThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.OrientationChangeThresholdForTeleport"),   TEXT("TeleportDetection.OrientationChangeThreshold"));

	return Info;
}

//======================================================================================================================
FRigUnit_SpawnPhysicsSolver_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}

	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnPhysicsSolver can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		PhysicsSolverComponentKey = Controller->AddComponent(
			FRigPhysicsSolverComponent::StaticStruct(), TEXT("PhysicsSolver"), Owner);
		if (PhysicsSolverComponentKey.IsValid())
		{
			if (FRigPhysicsSolverComponent* Component = Cast<FRigPhysicsSolverComponent>(
				ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
			{
				Component->SolverSettings = SolverSettings;
				Component->SpaceMotion = SpaceMotion;
				Component->TeleportDetection = TeleportDetection;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsSolverExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = Cast<FRigPhysicsSolverComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)) != nullptr;
}

//======================================================================================================================
FRigUnit_InstantiatePhysics_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (UControlRig* ControlRig = Cast<UControlRig>(Hierarchy.GetOuter()))
	{
		if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
			ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
		{
			if (FRigPhysicsSolver* PhysicsSolver = SolverComponent->GetPhysicsSolver())
			{
				PhysicsSolver->Instantiate(ExecuteContext, Hierarchy, *SolverComponent);
			}
		}
	}
}

//======================================================================================================================
FRigUnit_TrackInputPose_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
	{
		if (bForceNumberOfFrames)
		{
			SolverComponent->TrackInputCounter = NumberOfFrames;
		}
		else
		{
			SolverComponent->TrackInputCounter = FMath::Max(SolverComponent->TrackInputCounter, NumberOfFrames);
		}
	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_TrackInputPose::GetUpgradeInfo() const
{
	// No 1:1 upgrade target - the replacement lives on the Step Physics Solver node via Alpha = 0
	// + bTrackVelocitiesDuringPassThrough = false, which schedules an equivalent warm-up on resume.
	return FRigVMStructUpgradeInfo();
}

//======================================================================================================================
FRigUnit_StepPhysicsSolver_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	SCOPED_NAMED_EVENT(RigPhysicsSolver_StepSimulation, FColor::Yellow);

	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (CVarControlRigPhysicsEnableStepSolver.GetValueOnAnyThread() <= 0)
	{
		return;
	}

	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey));

	if (SolverComponent)
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(Hierarchy.GetOuter()))
		{
			// We only set this flag once we start executing, because that's when it matters.
			// Control rig is responsible for clearing it when the rig is constructed.
			ControlRig->SetContainsSimulation();

			if (FRigPhysicsSolver* PhysicsSolver = SolverComponent->GetPhysicsSolver())
			{
				const USceneComponent* OwningSceneComponent = ControlRig->GetOwningSceneComponent();
				const AActor* OwningActorPtr = OwningSceneComponent ? OwningSceneComponent->GetOwner() : nullptr;

				// Deprecated node has no bTrackVelocitiesDuringPassThrough pin - default to true
				// (smooth-resume behaviour), which is the closest match to the old "sim continues
				// silently at Alpha=0" semantics.
				PhysicsSolver->StepSimulation(
					ControlRig->GetWorld(), OwningActorPtr, ExecuteContext, Hierarchy, *SolverComponent,
					DeltaTimeOverride, SimulationSpaceDeltaTimeOverride, Alpha, /*bTrackVelocitiesDuringPassThrough=*/true);

				FRigPhysicsVisualizationSettings1 VisualizationSettings1 = VisualizationSettings;

				if (Alpha > 0.0f
					&& CVarControlRigPhysicsAllowVisualization.GetValueOnAnyThread() != 0
					&& RigPhysicsShouldVisualize(VisualizationSettings1))
				{
					PhysicsSolver->Draw(
						ExecuteContext.GetDrawInterface(), SolverComponent->SolverSettings,
						VisualizationSettings1, ExecuteContext.GetWorld());
				}
			}
		}
	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_StepPhysicsSolver::GetUpgradeInfo() const
{
	FRigUnit_StepPhysicsSolver1 NewNode;
	NewNode.PhysicsSolverComponentKey = PhysicsSolverComponentKey;
	NewNode.DeltaTimeOverride = DeltaTimeOverride;
	NewNode.SimulationSpaceDeltaTimeOverride = SimulationSpaceDeltaTimeOverride;
	NewNode.Alpha = Alpha;
	NewNode.VisualizationSettings = VisualizationSettings;
	return FRigVMStructUpgradeInfo(*this, NewNode);
}

//======================================================================================================================
FRigUnit_StepPhysicsSolver1_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	SCOPED_NAMED_EVENT(RigPhysicsSolver_StepSimulation, FColor::Yellow);

	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (CVarControlRigPhysicsEnableStepSolver.GetValueOnAnyThread() <= 0)
	{
		return;
	}

	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey));

	if (SolverComponent)
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(Hierarchy.GetOuter()))
		{
			// We only set this flag once we start executing, because that's when it matters.
			// Control rig is responsible for clearing it when the rig is constructed.
			ControlRig->SetContainsSimulation();

			if (FRigPhysicsSolver* PhysicsSolver = SolverComponent->GetPhysicsSolver())
			{
				const USceneComponent* OwningSceneComponent = ControlRig->GetOwningSceneComponent();
				const AActor* OwningActorPtr = OwningSceneComponent ? OwningSceneComponent->GetOwner() : nullptr;

				PhysicsSolver->StepSimulation(
					ControlRig->GetWorld(), OwningActorPtr, ExecuteContext, Hierarchy, *SolverComponent,
					DeltaTimeOverride, SimulationSpaceDeltaTimeOverride, Alpha, bTrackVelocitiesDuringPassThrough);

				if (Alpha > 0.0f
					&& CVarControlRigPhysicsAllowVisualization.GetValueOnAnyThread() != 0
					&& RigPhysicsShouldVisualize(VisualizationSettings))
				{
					PhysicsSolver->Draw(
						ExecuteContext.GetDrawInterface(), SolverComponent->SolverSettings,
						VisualizationSettings, ExecuteContext.GetWorld());
				}
			}
		}
	}
}

//======================================================================================================================
FRigUnit_SetPhysicsSolverSimulationSpaceSettings_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		ConvertLegacyPhysicsSimulationSpaceSettings(
			SimulationSpaceSettings, SolverComponent->SpaceMotion, SolverComponent->TeleportDetection);
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsSolverSimulationSpaceSettings_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (const FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		SimulationSpaceSettings = BuildLegacyPhysicsSimulationSpaceSettingsView(
			SolverComponent->SpaceMotion, SolverComponent->TeleportDetection);
	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_SetPhysicsSolverSimulationSpaceSettings::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo();
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_GetPhysicsSolverSimulationSpaceSettings::GetUpgradeInfo() const
{
	return FRigVMStructUpgradeInfo();
}

//======================================================================================================================
FRigUnit_SetPhysicsSolverSpaceMotion_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		SolverComponent->SpaceMotion = SpaceMotion;
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsSolverSpaceMotion_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (const FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		SpaceMotion = SolverComponent->SpaceMotion;
	}
}

//======================================================================================================================
FRigUnit_SetPhysicsSolverTeleportDetection_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		SolverComponent->TeleportDetection = TeleportDetection;
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsSolverTeleportDetection_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (const FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		TeleportDetection = SolverComponent->TeleportDetection;
	}
}

//======================================================================================================================
FRigUnit_SetPhysicsSolverExternalVelocity_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		Hierarchy.FindComponent(PhysicsSolverComponentKey)))
	{
		SolverComponent->SpaceMotion.Drag.ExternalLinearVelocity = ExternalLinearVelocity;
		SolverComponent->SpaceMotion.Drag.ExternalAngularVelocity = ExternalAngularVelocity;
		SolverComponent->SpaceMotion.Drag.ExternalTurbulenceVelocity = ExternalTurbulenceVelocity;
	}
}

//======================================================================================================================
FRigUnit_GetPhysicsSolverSpaceData_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
	{
		if (FRigPhysicsSolver* PhysicsSolver = SolverComponent->GetPhysicsSolver())
		{
			const FRigPhysicsSolver::FSimulationSpaceData& SimulationSpaceData = 
				PhysicsSolver->GetSimulationSpaceData();

			LinearVelocity = SimulationSpaceData.LinearVelocity;
			AngularVelocity = SimulationSpaceData.AngularVelocity;
			LinearAcceleration = SimulationSpaceData.LinearAcceleration;
			AngularAcceleration = SimulationSpaceData.AngularAcceleration;
			Gravity = SimulationSpaceData.Gravity;
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchySetPhysicsSolverAllowCCD_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigPhysicsSolverComponent* SolverComponent = Cast<FRigPhysicsSolverComponent>(
		ExecuteContext.Hierarchy->FindComponent(PhysicsSolverComponentKey)))
	{
		SolverComponent->SolverSettings.bAllowCCD = bAllowCCD;
	}
}
