// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolver.h"
#include "RigPhysicsSolver_Space.inl"

#include "RigPhysicsSolverComponent.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"

#include "PhysicsControlHelpers.h"

#include "Engine/Engine.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"

#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ShapeInstance.h"
#include "Chaos/Capsule.h"
#include "Chaos/PBDJointConstraintUtilities.h"

#include "PhysicsEngine/BodyInstance.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/BodySetup.h"

TAutoConsoleVariable<int> CVarControlRigPhysicsShowSimulationSpaceInfo(
	TEXT("ControlRig.Physics.ShowSimulationSpaceInfo"), 0,
	TEXT("Shows information associated with the simulation space used in control rig physics."));

//======================================================================================================================
void FRigPhysicsSolver::InitSimulationSpace(
	const FTransform& ComponentTM,
	const FTransform& BoneRelComponentTM)
{
	SimulationSpaceState.ComponentTM = ComponentTM;
	SimulationSpaceState.BoneRelComponentTM = BoneRelComponentTM;
}

//======================================================================================================================
// TODO We need support for double precision Perlin inputs - for now duplicate.
namespace FPerlinHelpers
{
	// random permutation of 256 numbers, repeated 2x
	static const int32 Permutation[512] = {
		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 222, 2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 117, 251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235,

		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 222, 2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 117, 251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235
	};

	// Gradient functions for 1D, 2D and 3D Perlin noise

	FORCEINLINE float Grad1(int32 Hash, double X)
	{
		// Slicing Perlin's 3D improved noise would give us only scales of -1, 0 and 1; this looks pretty bad so let's use a different sampling
		static const double Grad1Scales[16] = { -8 / 8, -7 / 8., -6 / 8., -5 / 8., -4 / 8., -3 / 8., -2 / 8., -1 / 8., 1 / 8., 2 / 8., 3 / 8., 4 / 8., 5 / 8., 6 / 8., 7 / 8., 8 / 8 };
		return Grad1Scales[Hash & 15] * X;
	}


	// Curve w/ second derivative vanishing at 0 and 1, from Perlin's improved noise paper
	FORCEINLINE double SmoothCurve(double X)
	{
		return X * X * X * (X * (X * 6.0 - 15.0) + 10.0);
	}

	FORCEINLINE float PerlinNoise1D(double X)
	{
		const double Xfl = FMath::FloorToFloat(X);
		const int64 Xi = (int64)(Xfl) & 255;
		X -= Xfl;
		const double Xm1 = X - 1.0;

		const int32 A = Permutation[Xi];
		const int32 B = Permutation[Xi + 1];

		const double U = SmoothCurve(X);

		// 2.0 factor to ensure (-1, 1) range
		return 2.0f * FMath::Lerp(Grad1(A, X), Grad1(B, Xm1), U);
	}
}

//======================================================================================================================
// Note - don't use the space conversion functions here as the state won't have been set yet.
FRigPhysicsSolver::FSimulationSpaceData FRigPhysicsSolver::UpdateSimulationSpaceStateAndCalculateData(
	const FRigVMExecuteContext&         ExecuteContext, 
	const URigHierarchy&                Hierarchy,
	const FRigPhysicsSolverComponent&   SolverComponent,
	const float                         Dt,
	const double                        AbsoluteTime)
{
	const FRigPhysicsSolverSettings& SolverSettings = SolverComponent.SolverSettings;
	const FRigPhysicsSimulationSpaceMotion& SpaceMotion = SolverComponent.SpaceMotion;
	const FRigPhysicsTeleportDetectionSettings& Teleport = SolverComponent.TeleportDetection;

	SimulationSpaceState.ComponentTM = ExecuteContext.GetToWorldSpaceTransform();

	if (SolverSettings.SimulationSpace == ERigPhysicsSimulationSpace::SpaceBone &&  SolverSettings.SpaceBone.IsValid())
	{
		SimulationSpaceState.BoneRelComponentTM = Hierarchy.GetGlobalTransform(SolverSettings.SpaceBone);
	}

	// Record the history - but avoid polluting it with zero Dt updates. What that means is - if we
	// get a zero-Dt update, then just update our current sim space TM, which means the time delta
	// from the previous state is actually the current Dt (i.e. don't overwrite the current Dt).
	if (Dt > SMALL_NUMBER)
	{
		SimulationSpaceState.PrevDt = SimulationSpaceState.Dt;
		SimulationSpaceState.Dt = Dt;

		SimulationSpaceState.PrevPrevSimulationSpaceTM = SimulationSpaceState.PrevSimulationSpaceTM;
		SimulationSpaceState.PrevSimulationSpaceTM = SimulationSpaceState.SimulationSpaceTM;
	}

	SimulationSpaceState.SimulationSpaceTM = GetSpaceTransform(
		SolverSettings.SimulationSpace, SimulationSpaceState.ComponentTM, SimulationSpaceState.BoneRelComponentTM);

	SimulationSpaceData = FSimulationSpaceData();
	SimulationSpaceData.Gravity = ::ConvertWorldVectorToSimSpaceNoScale(
		SolverSettings.SimulationSpace, SolverSettings.Gravity, 
		SimulationSpaceState.ComponentTM, SimulationSpaceState.BoneRelComponentTM);

	if (SolverSettings.SimulationSpace == ERigPhysicsSimulationSpace::World)
	{
		// TODO This is probably redundant unless we support runtime switching of the space
		InitSimulationSpace(SimulationSpaceState.ComponentTM, SimulationSpaceState.BoneRelComponentTM);
		SimulationSpaceData.LinearVelocity = SpaceMotion.Drag.ExternalLinearVelocity;
		SimulationSpaceData.AngularVelocity = FMath::DegreesToRadians(SpaceMotion.Drag.ExternalAngularVelocity);
		return SimulationSpaceData;
	}

	// If the timestep is zero, or if we have skipped an update, then it doesn't actually matter
	// what the velocity is - but make sure it doesn't corrupt anything.
	if (SimulationSpaceState.Dt < SMALL_NUMBER || UpdateCounter != PreviousUpdateCounter + 1)
	{
		SimulationSpaceData.LinearVelocity = FVector::ZeroVector;
		SimulationSpaceData.AngularVelocity = FVector::ZeroVector;
		SimulationSpaceData.LinearAcceleration = FVector::ZeroVector;
		SimulationSpaceData.AngularAcceleration = FVector::ZeroVector;
		return SimulationSpaceData;
	}

	// We calculate velocities etc in world space first, and then subsequently convert them into
	// simulation space.

	// Note that the velocity/accel calculations are intended to track the world/simulation behavior
	// - not necessarily be the most accurate calculations! For example, we could use one-sided
	// finite difference approximations, but this wouldn't necessarily be correct.

	// The two velocities are central differences whose midpoints are Dt/2 and Dt + PrevDt/2 before
	// "now", so they are (Dt + PrevDt)/2 apart - not Dt.
	const float AccelerationDt = 0.5f * (SimulationSpaceState.Dt + SimulationSpaceState.PrevDt);

	// World-space component linear velocity and acceleration
	SimulationSpaceData.LinearVelocity = UE::PhysicsControl::CalculateLinearVelocity(
		SimulationSpaceState.PrevSimulationSpaceTM.GetTranslation(),
		SimulationSpaceState.SimulationSpaceTM.GetTranslation(), SimulationSpaceState.Dt);
	const FVector PrevSpaceLinearVel =
		SimulationSpaceState.PrevDt < SMALL_NUMBER
		? SimulationSpaceData.LinearVelocity
		: UE::PhysicsControl::CalculateLinearVelocity(
			SimulationSpaceState.PrevPrevSimulationSpaceTM.GetTranslation(),
			SimulationSpaceState.PrevSimulationSpaceTM.GetTranslation(), SimulationSpaceState.PrevDt);
	SimulationSpaceData.LinearAcceleration =
		(SimulationSpaceData.LinearVelocity - PrevSpaceLinearVel) / AccelerationDt;

	// World-space component angular velocity and acceleration
	SimulationSpaceData.AngularVelocity = UE::PhysicsControl::CalculateAngularVelocity(
		SimulationSpaceState.PrevSimulationSpaceTM.GetRotation(),
		SimulationSpaceState.SimulationSpaceTM.GetRotation(), SimulationSpaceState.Dt);
	const FVector PrevSpaceAngularVel =
		SimulationSpaceState.PrevDt < SMALL_NUMBER
		? SimulationSpaceData.AngularVelocity
		: UE::PhysicsControl::CalculateAngularVelocity(
			SimulationSpaceState.PrevPrevSimulationSpaceTM.GetRotation(),
			SimulationSpaceState.PrevSimulationSpaceTM.GetRotation(), SimulationSpaceState.PrevDt);
	SimulationSpaceData.AngularAcceleration =
		(SimulationSpaceData.AngularVelocity - PrevSpaceAngularVel) / AccelerationDt;

	// Apply Z scale (applies to both linear velocity AND acceleration)
	SimulationSpaceData.LinearVelocity.Z *= SpaceMotion.VerticalMotionScale;
	SimulationSpaceData.LinearAcceleration.Z *= SpaceMotion.VerticalMotionScale;

	bool bLinearAccelerationTrigger = Teleport.bFromLinearAcceleration &&
		SimulationSpaceData.LinearAcceleration.SquaredLength() >
		FMath::Square(Teleport.LinearAccelerationThreshold);
	bool bAngularAccelerationTrigger = Teleport.bFromAngularAcceleration &&
		SimulationSpaceData.AngularAcceleration.SquaredLength() >
		FMath::Square(FMath::DegreesToRadians(Teleport.AngularAccelerationThreshold));
	bool bPositionTrigger = Teleport.bFromPositionChange &&
		SimulationSpaceData.LinearVelocity.SquaredLength() >
		FMath::Square(Teleport.PositionChangeThreshold / SimulationSpaceState.Dt);
	bool bOrientationTrigger = Teleport.bFromOrientationChange &&
		SimulationSpaceData.AngularVelocity.SquaredLength() >
		FMath::Square(FMath::DegreesToRadians(
			Teleport.OrientationChangeThreshold / SimulationSpaceState.Dt));

	// Check and report teleports
	if (bLinearAccelerationTrigger || bAngularAccelerationTrigger || bPositionTrigger || bOrientationTrigger)
	{
		if (bLinearAccelerationTrigger)
		{
			UE_LOGF(LogRigPhysics, Log, "Detected linear Acceleration (%f > %f) teleport",
				SimulationSpaceData.LinearAcceleration.Length(),
				Teleport.LinearAccelerationThreshold);
		}
		if (bAngularAccelerationTrigger)
		{
			UE_LOGF(LogRigPhysics, Log, "Detected angular Acceleration (%f > %f) teleport",
				FMath::RadiansToDegrees(SimulationSpaceData.AngularAcceleration.Length()),
				Teleport.AngularAccelerationThreshold);
		}
		if (bPositionTrigger)
		{
			UE_LOGF(LogRigPhysics, Log, "Detected position (%f > %f) teleport",
				SimulationSpaceData.LinearVelocity.Length() * SimulationSpaceState.Dt,
				Teleport.PositionChangeThreshold);
		}
		if (bOrientationTrigger)
		{
			UE_LOGF(LogRigPhysics, Log, "Detected orientation (%f > %f) teleport",
				FMath::RadiansToDegrees(SimulationSpaceData.AngularVelocity.Length() * SimulationSpaceState.Dt),
				Teleport.OrientationChangeThreshold);
		}

		// Note that a teleport detection shouldn't change the pose, or the current motion. We just
		// don't want to bring in that unwanted global motion.
		SimulationSpaceData.LinearVelocity = FVector::ZeroVector;
		SimulationSpaceData.AngularVelocity = FVector::ZeroVector;
		SimulationSpaceData.LinearAcceleration = FVector::ZeroVector;
		SimulationSpaceData.AngularAcceleration = FVector::ZeroVector;

		// This will stop the next step from using bogus values too.
		SimulationSpaceState.PrevSimulationSpaceTM = SimulationSpaceState.SimulationSpaceTM;
		SimulationSpaceState.PrevPrevSimulationSpaceTM = SimulationSpaceState.SimulationSpaceTM;
		SimulationSpaceState.PrevDt = 0;
		SimulationSpaceState.Dt = 0;
		// Avoid cached transforms being used in controls by bumping the update counter. 
		UpdateCounter += 1;
	}
	else
	{
		if (SpaceMotion.bClampLinearVelocity)
		{
			SimulationSpaceData.LinearVelocity =
				SimulationSpaceData.LinearVelocity.GetClampedToMaxSize(SpaceMotion.MaxLinearVelocity);
		}
		if (SpaceMotion.bClampAngularVelocity)
		{
			SimulationSpaceData.AngularVelocity =
				SimulationSpaceData.AngularVelocity.GetClampedToMaxSize(SpaceMotion.MaxAngularVelocity);
		}
		if (SpaceMotion.bClampLinearAcceleration)
		{
			SimulationSpaceData.LinearAcceleration =
				SimulationSpaceData.LinearAcceleration.GetClampedToMaxSize(SpaceMotion.MaxLinearAcceleration);
		}
		if (SpaceMotion.bClampAngularAcceleration)
		{
			SimulationSpaceData.AngularAcceleration =
				SimulationSpaceData.AngularAcceleration.GetClampedToMaxSize(SpaceMotion.MaxAngularAcceleration);
		}
	}

	SimulationSpaceData.LinearVelocity += SpaceMotion.Drag.ExternalLinearVelocity;
	SimulationSpaceData.AngularVelocity += FMath::DegreesToRadians(SpaceMotion.Drag.ExternalAngularVelocity);

	if (!SpaceMotion.Drag.ExternalTurbulenceVelocity.IsNearlyZero())
	{
		FVector T(
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime),
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime + 10.0),
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime + 20.0));

		FVector Turbulence = T * SpaceMotion.Drag.ExternalTurbulenceVelocity;

		SimulationSpaceData.LinearVelocity += Turbulence;

		if (GEngine && CVarControlRigPhysicsShowSimulationSpaceInfo.GetValueOnAnyThread())
		{
			GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, FColor::Yellow,
				FString::Printf(TEXT("Turbulence %s"), *Turbulence.ToString()));
		}
	}

	// Transform world-space motion into simulation space TODO note that this matches the code
	// in RBAN, and is doing what the interface requires (i.e. movement of the space in the space of
	// the space!). TODO note that we don't currently support scaling
	SimulationSpaceData.LinearVelocity =
		SimulationSpaceState.SimulationSpaceTM.InverseTransformVectorNoScale(SimulationSpaceData.LinearVelocity);
	SimulationSpaceData.AngularVelocity =
		SimulationSpaceState.SimulationSpaceTM.InverseTransformVectorNoScale(SimulationSpaceData.AngularVelocity);
	SimulationSpaceData.LinearAcceleration =
		SimulationSpaceState.SimulationSpaceTM.InverseTransformVectorNoScale(SimulationSpaceData.LinearAcceleration);
	SimulationSpaceData.AngularAcceleration =
		SimulationSpaceState.SimulationSpaceTM.InverseTransformVectorNoScale(SimulationSpaceData.AngularAcceleration);

	if (GEngine && CVarControlRigPhysicsShowSimulationSpaceInfo.GetValueOnAnyThread())
	{
		GEngine->AddOnScreenDebugMessage(INDEX_NONE, 0.f, FColor::Yellow,
			FString::Printf(TEXT("Dt %6.2fms Space P %35s V %35s A %35s"),
				Dt * 1000.0f,
				*SimulationSpaceState.SimulationSpaceTM.GetLocation().ToString(),
				*SimulationSpaceData.LinearVelocity.ToString(),
				*SimulationSpaceData.LinearAcceleration.ToString()));
	}

	return SimulationSpaceData;
}
