// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsSolver.h"
#include "PhysicsControlHelpers.h"

#include "Engine/Engine.h"

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::GetSpaceTransform(
	ERigPhysicsSimulationSpace Space, const FTransform& ComponentTM, const FTransform& BoneTM)
{
	switch (Space)
	{
	case ERigPhysicsSimulationSpace::Component: return ComponentTM;
	case ERigPhysicsSimulationSpace::World: return FTransform::Identity;
	case ERigPhysicsSimulationSpace::SpaceBone: return BoneTM * ComponentTM;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return FTransform::Identity;
	}
}

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::GetSimulationSpaceTransform(const FRigPhysicsSolverSettings& SolverSettings) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return SimulationSpaceState.ComponentTM;
	case ERigPhysicsSimulationSpace::World: return FTransform::Identity;
	case ERigPhysicsSimulationSpace::SpaceBone: return 
		SimulationSpaceState.BoneRelComponentTM * SimulationSpaceState.ComponentTM;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return FTransform::Identity;
	}
}

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::ConvertComponentSpaceTransformToSimSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FTransform& TM) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return TM;
	case ERigPhysicsSimulationSpace::World: return TM * SimulationSpaceState.ComponentTM;
	case ERigPhysicsSimulationSpace::SpaceBone: return TM.GetRelativeTransform(SimulationSpaceState.BoneRelComponentTM);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return TM;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertComponentSpacePositionToSimSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& P) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return P;
	case ERigPhysicsSimulationSpace::World:  return SimulationSpaceState.ComponentTM.TransformPositionNoScale(P);
	case ERigPhysicsSimulationSpace::SpaceBone: return SimulationSpaceState.BoneRelComponentTM.InverseTransformPositionNoScale(P);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return P;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertComponentSpaceVectorToSimSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& V) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return V;
	case ERigPhysicsSimulationSpace::World: return SimulationSpaceState.ComponentTM.TransformVectorNoScale(V);
	case ERigPhysicsSimulationSpace::SpaceBone: 
		return SimulationSpaceState.BoneRelComponentTM.InverseTransformVectorNoScale(V);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return V;
	}
}

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::ConvertSimSpaceTransformToComponentSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FTransform& TM) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return TM;
	case ERigPhysicsSimulationSpace::World: return TM.GetRelativeTransform(SimulationSpaceState.ComponentTM);
	case ERigPhysicsSimulationSpace::SpaceBone: return TM * SimulationSpaceState.BoneRelComponentTM;
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return TM;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertSimSpacePositionToComponentSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& Position) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return Position;
	case ERigPhysicsSimulationSpace::World: return SimulationSpaceState.ComponentTM.InverseTransformPositionNoScale(Position);
	case ERigPhysicsSimulationSpace::SpaceBone: return SimulationSpaceState.BoneRelComponentTM.TransformPositionNoScale(Position);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return Position;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertSimSpaceVectorToComponentSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& Vector) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return Vector;
	case ERigPhysicsSimulationSpace::World: return SimulationSpaceState.ComponentTM.InverseTransformVectorNoScale(Vector);
	case ERigPhysicsSimulationSpace::SpaceBone: return SimulationSpaceState.BoneRelComponentTM.TransformVectorNoScale(Vector);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return Vector;
	}
}

//======================================================================================================================
FORCEINLINE FVector ConvertWorldVectorToSimSpaceNoScale(
	ERigPhysicsSimulationSpace Space, const FVector& WorldVector, 
	const FTransform& ComponentTM, const FTransform& BoneTM)
{
	switch (Space)
	{
	case ERigPhysicsSimulationSpace::Component: return ComponentTM.InverseTransformVectorNoScale(WorldVector);
	case ERigPhysicsSimulationSpace::World: return WorldVector;
	case ERigPhysicsSimulationSpace::SpaceBone:
		return BoneTM.InverseTransformVectorNoScale(ComponentTM.InverseTransformVectorNoScale(WorldVector));
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return WorldVector;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertWorldVectorToSimSpaceNoScale(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& WorldVector) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: 
		return SimulationSpaceState.ComponentTM.InverseTransformVectorNoScale(WorldVector);
	case ERigPhysicsSimulationSpace::World: return WorldVector;
	case ERigPhysicsSimulationSpace::SpaceBone:
		return SimulationSpaceState.BoneRelComponentTM.InverseTransformVectorNoScale(
			SimulationSpaceState.ComponentTM.InverseTransformVectorNoScale(WorldVector));
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return WorldVector;
	}
}

//======================================================================================================================
FORCEINLINE FVector FRigPhysicsSolver::ConvertWorldPositionToSimSpaceNoScale(
	const FRigPhysicsSolverSettings& SolverSettings, const FVector& WorldPosition) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: 
		return SimulationSpaceState.ComponentTM.InverseTransformPositionNoScale(WorldPosition);
	case ERigPhysicsSimulationSpace::World: return WorldPosition;
	case ERigPhysicsSimulationSpace::SpaceBone:
		return SimulationSpaceState.BoneRelComponentTM.InverseTransformPositionNoScale(
			SimulationSpaceState.ComponentTM.InverseTransformPositionNoScale(WorldPosition));
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return WorldPosition;
	}
}

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::ConvertWorldTransformToSimSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FTransform& WorldTM) const
{
	switch (SolverSettings.SimulationSpace)
	{
	case ERigPhysicsSimulationSpace::Component: return WorldTM.GetRelativeTransform(SimulationSpaceState.ComponentTM);
	case ERigPhysicsSimulationSpace::World: return WorldTM;
	case ERigPhysicsSimulationSpace::SpaceBone:
		return WorldTM.GetRelativeTransform(SimulationSpaceState.BoneRelComponentTM * SimulationSpaceState.ComponentTM);
	default: ensureMsgf(false, TEXT("Unsupported Simulation Space")); return WorldTM;
	}
}

//======================================================================================================================
FORCEINLINE FTransform FRigPhysicsSolver::ConvertCollisionSpaceTransformToSimSpace(
	const FRigPhysicsSolverSettings& SolverSettings, const FTransform& TM) const
{
	FTransform SimSpaceTM = GetSpaceTransform(
		SolverSettings.SimulationSpace, SimulationSpaceState.ComponentTM, SimulationSpaceState.BoneRelComponentTM);
	FTransform CollisionSpaceTM = GetSpaceTransform(
		SolverSettings.CollisionSpace, SimulationSpaceState.ComponentTM, SimulationSpaceState.BoneRelComponentTM);
	FTransform WorldSpaceTM = TM * CollisionSpaceTM;
	return WorldSpaceTM.GetRelativeTransform(SimSpaceTM);
}
