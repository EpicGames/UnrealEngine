// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolver.h"
#include "RigPhysicsSolver_Space.inl"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"

#include "RigVMCore/RigVMDrawInterface.h"

#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsJointHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"

#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ShapeInstance.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/PBDJointConstraintUtilities.h"

#include "Engine/Engine.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Math/Color.h"

// TODO expose these in the visualization settings
namespace RigPhysicsSolverDraw
{
	FLinearColor DynamicCollisionColor = FColor::Yellow;
	FLinearColor DynamicNoCollisionColor = FColor::Orange;
	FLinearColor KinematicCollisionColor = FColor::Blue;
	FLinearColor KinematicNoCollisionColor = FColor::Purple;

	FLinearColor ActiveContactColor = FColor::Red;
	FLinearColor InactiveContactColor = FColor::Silver;
}

TAutoConsoleVariable<int> CVarControlRigPhysicsShowActiveContactsOverride(
	TEXT("ControlRig.Physics.ShowActiveContactsOverride"), -1,
	TEXT("Whether to draw active contacts (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowInactiveContactsOverride(
	TEXT("ControlRig.Physics.ShowInactiveContactsOverride"), -1,
	TEXT("Whether to draw inactive contacts (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowBodiesOverride(
	TEXT("ControlRig.Physics.ShowBodiesOverride"), -1,
	TEXT("Whether to draw bodies (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowCentreOfMassOverride(
	TEXT("ControlRig.Physics.ShowCentreOfMassOverride"), -1,
	TEXT("Whether to draw centres of mass (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowJointsOverride(
	TEXT("ControlRig.Physics.ShowJointsOverride"), -1,
	TEXT("Whether to draw joints (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowControlsOverride(
	TEXT("ControlRig.Physics.ShowControlsOverride"), -1,
	TEXT("Whether to draw controls (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowWorldObjectsOverride(
	TEXT("ControlRig.Physics.ShowWorldObjectsOverride"), -1,
	TEXT("Whether to draw world objects (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigPhysicsShowWorldOverlapBoxOverride(
	TEXT("ControlRig.Physics.ShowWorldOverlapBoxOverride"), -1,
	TEXT("Whether to draw the world overlap box (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

//======================================================================================================================
// Tri-state helper for the Show*Override CVars: -1 defers to the authored setting, 0 forces off, anything else forces on.
inline bool GetControlRigPhysicsShow(const bool SettingsValue, const TAutoConsoleVariable<int>& CVar)
{
	const int32 ShowOverride = CVar.GetValueOnAnyThread();
	return ShowOverride < 0 ? SettingsValue : (ShowOverride != 0);
}

//======================================================================================================================
// True if any per-element draw would actually emit geometry, given the rig's bShow* flags AND the
// per-element Show*Override CVars. Used as the early-out for Draw() in the runtime gate so we don't
// pay the cost of entering Draw (which also enables low-level Chaos debug draw) when nothing visible
// would come out the other side.
bool RigPhysicsShouldVisualize(const FRigPhysicsVisualizationSettings1& VisualizationSettings)
{
	return GetControlRigPhysicsShow(
		VisualizationSettings.bShowBodies, CVarControlRigPhysicsShowBodiesOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowCentreOfMass, CVarControlRigPhysicsShowCentreOfMassOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowJoints, CVarControlRigPhysicsShowJointsOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowControls, CVarControlRigPhysicsShowControlsOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowWorldObjects, CVarControlRigPhysicsShowWorldObjectsOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowWorldOverlapBox, CVarControlRigPhysicsShowWorldOverlapBoxOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowActiveContacts, CVarControlRigPhysicsShowActiveContactsOverride)
		|| GetControlRigPhysicsShow(
			VisualizationSettings.bShowInactiveContacts, CVarControlRigPhysicsShowInactiveContactsOverride);
}

//======================================================================================================================
static FLinearColor GetColor(bool bIsKinematic, bool bCollides)
{
	if (bIsKinematic)
	{
		return bCollides 
			? RigPhysicsSolverDraw::KinematicCollisionColor : RigPhysicsSolverDraw::KinematicNoCollisionColor;
	}
	else
	{
		return bCollides 
			? RigPhysicsSolverDraw::DynamicCollisionColor : RigPhysicsSolverDraw::DynamicNoCollisionColor;
	}
}

//======================================================================================================================
static void DrawShapes(
	FRigVMDrawInterface&                     DI,
	const FRigPhysicsVisualizationSettings1& VisualizationSettings,
	const bool                               bIsKinematic,
	const bool                               bCollides,
	const Chaos::FGeometryParticleHandle*    Particle,
	const FTransform&                        ShapeTransform, 
	const Chaos::FImplicitObject*            ImplicitObject,
	const Chaos::FPerShapeData*              Shape)
{
	using namespace Chaos;

	if (!ImplicitObject)
	{
		return;
	}
	const EImplicitObjectType PackedType = ImplicitObject->GetType(); // Type includes scaling and instancing data
	const EImplicitObjectType InnerType = GetInnerType(ImplicitObject->GetType());

	// For simplicity, we're going to assume no scaling, instancing, transforming etc

	if (!ImplicitObject)
	{
		return;
	}

	switch (InnerType)
	{
	case ImplicitObjectType::Transformed:
	{
		const TImplicitObjectTransformed<FReal, 3>* Transformed = 
			ImplicitObject->GetObject<TImplicitObjectTransformed<FReal, 3>>();
		FTransform TransformedTransform = FTransform(
			ShapeTransform.GetRotation() * Transformed->GetTransform().GetRotation(),
			ShapeTransform.TransformPosition(Transformed->GetTransform().GetLocation()));
		DrawShapes(
			DI, VisualizationSettings, bIsKinematic, bCollides, Particle, 
			TransformedTransform, Transformed->GetTransformedObject(), Shape);
		return;
	}
	case ImplicitObjectType::Sphere:
	{
		const Chaos::FSphere* Sphere = ImplicitObject->GetObject<Chaos::FSphere>();
		DI.DrawSphere(ShapeTransform, FTransform(FVec3(Sphere->GetCenterOfMass())), Sphere->GetRadiusf(),
			GetColor(bIsKinematic, bCollides),
			VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail);
		break;
	}
	case ImplicitObjectType::Box:
	{
		const TBox<FReal, 3>* Box = ImplicitObject->GetObject<TBox<FReal, 3>>(); // Axis aligned
		DI.DrawBox(
			FTransform(FQuat::Identity, Box->GetCenter()) * ShapeTransform, 
			FTransform(FQuat::Identity, FVector::ZeroVector, Box->Extents()),
			GetColor(bIsKinematic, bCollides),
			VisualizationSettings.LineThickness);
		break;
	}
	case ImplicitObjectType::Capsule:
	{
		const Chaos::FCapsule* Capsule = ImplicitObject->GetObject<Chaos::FCapsule>();
		const FQuat Q = FRotationMatrix::MakeFromZ(Capsule->GetAxis()).ToQuat();
		DI.DrawCapsule(ShapeTransform, FTransform(Q, Capsule->GetCenterOfMass()),
			Capsule->GetRadiusf(), Capsule->GetHeightf(),
			GetColor(bIsKinematic, bCollides),
			VisualizationSettings.LineThickness);
		break;
	}
	case ImplicitObjectType::Convex:
	{
		// Convex shapes arrive wrapped in TImplicitObjectInstanced<FConvex> (Instanced or Scaled) by
		// ChaosInterface::CreateGeometry, so go through the wrapper to reach the hull.
		const FConvex* Convex = nullptr;
		if (IsInstanced(PackedType))
		{
			if (const auto* Wrapped = ImplicitObject->GetObject<TImplicitObjectInstanced<FConvex>>())
			{
				Convex = Wrapped->GetInstancedObject();
			}
		}
		else if (IsScaled(PackedType))
		{
			if (const auto* Wrapped = ImplicitObject->GetObject<TImplicitObjectScaled<FConvex>>())
			{
				Convex = Wrapped->GetUnscaledObject();
			}
		}
		else
		{
			Convex = ImplicitObject->GetObject<FConvex>();
		}

		if (Convex)
		{
			const int32 NumEdges = Convex->NumEdges();
			TArray<FVector> Endpoints;
			Endpoints.Reserve(NumEdges * 2);
			for (int32 EdgeIndex = 0; EdgeIndex < NumEdges; ++EdgeIndex)
			{
				Endpoints.Add(FVector(Convex->GetVertex(Convex->GetEdgeVertex(EdgeIndex, 0))));
				Endpoints.Add(FVector(Convex->GetVertex(Convex->GetEdgeVertex(EdgeIndex, 1))));
			}
			DI.DrawLines(
				ShapeTransform, Endpoints,
				GetColor(bIsKinematic, bCollides),
				VisualizationSettings.LineThickness);
		}
		break;
	}
	default:
		// If we don't know what it is, don't draw it.
		break;
	}
}

//======================================================================================================================
static void DrawActor(
	FRigVMDrawInterface&                     DI,
	const FRigPhysicsVisualizationSettings1& VisualizationSettings,
	const bool                               bShowCentreOfMass,
	const FTransform&                        SpaceTransform,
	const ImmediatePhysics::FActorHandle&    ActorHandle)
{
	bool bIsKinematic = ActorHandle.GetIsKinematic();
	bool bCollides = ActorHandle.GetHasCollision();

	const Chaos::FGeometryParticleHandle* GeometryParticleHandle = ActorHandle.GetParticle();
	const FTransform ParticleTransform = GeometryParticleHandle->GetTransformXR() * SpaceTransform;
	for (const Chaos::FShapeInstancePtr& ShapeInstance : GeometryParticleHandle->ShapeInstances())
	{
		const Chaos::FImplicitObject* ImplicitObject = ShapeInstance->GetGeometry();
		DrawShapes(
			DI, VisualizationSettings, bIsKinematic, bCollides, GeometryParticleHandle,
			ParticleTransform, ImplicitObject, ShapeInstance.Get());
	}

	if (bShowCentreOfMass)
	{
		float Size = 1.0f * VisualizationSettings.ShapeSize;
		DI.DrawAxes(
			ParticleTransform, ActorHandle.GetLocalCoMTransform(),
			Size, VisualizationSettings.LineThickness);
	}
}

//======================================================================================================================
enum
{
	TwistIndex = 0,
	Swing1Index = 1,
	Swing2Index = 2
};

//======================================================================================================================
// Returns the limit range - it will be the value passed in unless it the motion is free or locked. 
float GetLimitAngleRadians(float LimitAngle, Chaos::EJointMotionType LimitType)
{
	switch (LimitType)
	{
	case Chaos::EJointMotionType::Free: return UE_PI;
	case Chaos::EJointMotionType::Locked: return 0.f;
	default: return LimitAngle;
	}
}

//======================================================================================================================
static void DrawJoint(
	FRigVMDrawInterface&                     DI,
	const FRigPhysicsVisualizationSettings1& VisualizationSettings,
	const FTransform&                        SpaceTransform,
	const ImmediatePhysics::FJointHandle&    JointHandle,
	const bool                               bDrawAsAxes)
{
	const Chaos::FPBDJointConstraintHandle* ConstraintHandle = JointHandle.GetConstraint();
	if (!ConstraintHandle)
	{
		return;
	}

	const Chaos::TVec2<const ImmediatePhysics_Chaos::FActorHandle*>& ActorHandles = JointHandle.GetActorHandles();
	const ImmediatePhysics_Chaos::FActorHandle* ChildActor = ActorHandles[0];
	const ImmediatePhysics_Chaos::FActorHandle* ParentActor = ActorHandles[1];

	FTransform ChildActorTM = (ChildActor ? ChildActor->GetWorldTransform() : FTransform()) * SpaceTransform;
	FTransform ParentActorTM = (ParentActor ? ParentActor->GetWorldTransform() : FTransform()) * SpaceTransform;

	bool bChildIsKinematic = ChildActor ? ChildActor->GetIsKinematic() : true;
	bool bParentIsKinematic = ParentActor ? ParentActor->GetIsKinematic() : true;

	float Size = 5.0f * VisualizationSettings.ShapeSize;

	const Chaos::FPBDJointSettings& JointSettings = ConstraintHandle->GetSettings();

	FTransform DialFrame = JointSettings.ConnectorTransforms[0] * ChildActorTM;
	FTransform LimitFrame = JointSettings.ConnectorTransforms[1] * ParentActorTM;

	if (bDrawAsAxes)
	{
		DI.DrawAxes(
			FTransform(JointSettings.AngularDrivePositionTarget, JointSettings.LinearDrivePositionTarget) * LimitFrame, 
			FTransform(), Size, VisualizationSettings.LineThickness * 2);
		DI.DrawAxes(DialFrame, FTransform(), Size, VisualizationSettings.LineThickness);
	}

	{
		// See FConstraintInstance::DrawConstraintImp for inspiration

		// There seems to be a swap between swing1 and swing2 compared to GetSwingTwistAngles
		FVector LimitAngleRadians(
			GetLimitAngleRadians(JointSettings.AngularLimits[0], JointSettings.AngularMotionTypes[0]),
			GetLimitAngleRadians(JointSettings.AngularLimits[2], JointSettings.AngularMotionTypes[2]),
			GetLimitAngleRadians(JointSettings.AngularLimits[1], JointSettings.AngularMotionTypes[1])
		);

		Chaos::FReal TwistAngle, Swing1Angle, Swing2Angle;
		const FQuat LimitQ = LimitFrame.GetRotation();
		FQuat DialQ = DialFrame.GetRotation();
		DialQ.EnforceShortestArcWith(LimitQ);
		Chaos::FPBDJointUtilities::GetSwingTwistAngles(LimitQ, DialQ, TwistAngle, Swing1Angle, Swing2Angle);

		bool bDrawViolatedLimits = true;

		bool bTwistViolated = false;
		bool bSwing1Violated = false;
		bool bSwing2Violated = false;

		if (bDrawViolatedLimits)
		{
			bTwistViolated = JointSettings.AngularMotionTypes[TwistIndex] == Chaos::EJointMotionType::Limited &&
				FMath::Abs(TwistAngle) > LimitAngleRadians[0];

			bSwing1Violated = JointSettings.AngularMotionTypes[Swing1Index] == Chaos::EJointMotionType::Limited &&
				FMath::Abs(Swing1Angle) > LimitAngleRadians[1];

			bSwing2Violated = JointSettings.AngularMotionTypes[Swing2Index] == Chaos::EJointMotionType::Limited &&
				FMath::Abs(Swing2Angle) > LimitAngleRadians[2];
		}

		const bool bLockSwing1 = JointSettings.AngularMotionTypes[Swing1Index] == Chaos::EJointMotionType::Locked;
		const bool bLockSwing2 = JointSettings.AngularMotionTypes[Swing2Index] == Chaos::EJointMotionType::Locked;
		const bool bLockAllSwing = bLockSwing1 && bLockSwing2;

		check(GEngine->ConstraintLimitMaterialX && GEngine->ConstraintLimitMaterialY && GEngine->ConstraintLimitMaterialZ);
		static UMaterialInterface* LimitMaterialX = GEngine->ConstraintLimitMaterialX;
		static UMaterialInterface* LimitMaterialXAxis = GEngine->ConstraintLimitMaterialXAxis;
		static UMaterialInterface* LimitMaterialY = GEngine->ConstraintLimitMaterialY;
		static UMaterialInterface* LimitMaterialYAxis = GEngine->ConstraintLimitMaterialYAxis;
		static UMaterialInterface* LimitMaterialZ = GEngine->ConstraintLimitMaterialZ;
		static UMaterialInterface* LimitMaterialZAxis = GEngine->ConstraintLimitMaterialZAxis;

		float ArrowLength = Size * 1.05f; // stick out a little bit 

		// If swing is limited (but not locked) - draw the swing limit cone.
		if (!bLockAllSwing)
		{
			if (JointSettings.AngularMotionTypes[Swing1Index] == Chaos::EJointMotionType::Free && 
				JointSettings.AngularMotionTypes[Swing2Index] == Chaos::EJointMotionType::Free)
			{
				DI.DrawSphere(LimitFrame, FTransform(), Size * 0.2f, FColor::White, 
					VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail);
			}
			else
			{
				FTransform ConeLimitTM = LimitFrame;
				ConeLimitTM.SetScale3D(FVector(Size));
				const float Swing1LimitAngle = LimitAngleRadians[Swing1Index];
				const float Swing2LimitAngle = LimitAngleRadians[Swing2Index];
				DI.DrawCone(
					ConeLimitTM, FTransform(), Swing1LimitAngle, Swing2LimitAngle, VisualizationSettings.ShapeDetail, 
					true, FColor::Red, LimitMaterialX->GetRenderProxy(), VisualizationSettings.LineThickness);
			}
			// Draw the swing Dial indicator - shows the current orientation of the child frame
			// relative to the parent frame on the swing axis.
			// Start the arrow at the limit position as it can be confusing if there is joint separation
			DI.DrawArrow(FTransform(LimitFrame.GetTranslation()), 
				DialFrame.GetUnitAxis(EAxis::X) * ArrowLength,
				DialFrame.GetUnitAxis(EAxis::Y), FColor::Red, VisualizationSettings.LineThickness);

			if (bSwing1Violated || bSwing2Violated)
			{
				// Drawing a sphere seems a bit heavy... but it appears that if DrawPoint is used then they get
				// culled when you get close!
				DI.DrawSphere(FTransform(
					DialFrame.GetRotation(),
					LimitFrame.GetTranslation() + DialFrame.GetUnitAxis(EAxis::X) * ArrowLength),
					FTransform(),
					ArrowLength * 0.01f, FColor::Orange, VisualizationSettings.LineThickness * 4);
			}

		}

		// Draw the twist limit
		if (JointSettings.AngularMotionTypes[TwistIndex] != Chaos::EJointMotionType::Locked)
		{
			// Draw as a flat cone
			FTransform ConeLimitTM = LimitFrame;
			ConeLimitTM.SetScale3D(FVector(Size));
			const float TwistLimitAngle = LimitAngleRadians[TwistIndex];
			DI.DrawCone(ConeLimitTM, FTransform(
				FQuat::MakeFromRotationVector(FVector(0, 1, 0) * -HALF_PI) * 
				FQuat::MakeFromRotationVector(FVector(0, 0, 1) * HALF_PI)),
				TwistLimitAngle, 0.0f, VisualizationSettings.ShapeDetail, true, FColor::Green,
				LimitMaterialYAxis->GetRenderProxy(), VisualizationSettings.LineThickness);

			// Draw the twist Dial indicator - shows the current orientation of the child frame
			// relative to the parent frame on the twist axis.
			FQuat Rot(LimitFrame.GetUnitAxis(EAxis::X), TwistAngle);
			FVector TwistArrow = Rot * LimitFrame.GetUnitAxis(EAxis::Y);
			FVector TwistSideDir = FVector::CrossProduct(LimitFrame.GetUnitAxis(EAxis::Z), TwistArrow);

			DI.DrawArrow(FTransform(LimitFrame.GetTranslation()),
				TwistArrow * ArrowLength,
				TwistSideDir, FColor::Green, VisualizationSettings.LineThickness);

			if (bTwistViolated)
			{
				DI.DrawSphere(FTransform(
					DialFrame.GetRotation(),
					LimitFrame.GetTranslation() + TwistArrow * ArrowLength),
					FTransform(),
					ArrowLength * 0.01f, FColor::Orange, VisualizationSettings.LineThickness * 4);
			}
		}
	}
}

//======================================================================================================================
static bool IsConstraintDriveActive(const Chaos::FPBDJointConstraintHandle& Constraint)
{
	const Chaos::FPBDJointSettings& Settings = Constraint.GetSettings();
	if (Settings.bAngularSLerpPositionDriveEnabled || Settings.bAngularSwingPositionDriveEnabled || Settings.bAngularTwistPositionDriveEnabled)
	{
		if (Settings.AngularDriveStiffness.SizeSquared() > 0.0)
		{
			return true;
		}
	}
	if (Settings.bAngularSLerpVelocityDriveEnabled || Settings.bAngularSwingVelocityDriveEnabled || Settings.bAngularTwistVelocityDriveEnabled)
	{
		if (Settings.AngularDriveDamping.SizeSquared() > 0.0)
		{
			return true;
		}
	}
	if (Settings.bLinearPositionDriveEnabled.Max() && Settings.LinearDriveStiffness.SizeSquared() > 0.0)
	{
		return true;
	}
	if (Settings.bLinearVelocityDriveEnabled.Max() && Settings.LinearDriveDamping.SizeSquared() > 0.0)
	{
		return true;
	}
	return false;
}

//======================================================================================================================
void FRigPhysicsSolver::Draw(
	FRigVMDrawInterface*                     DI,
	const FRigPhysicsSolverSettings&         SolverSettings,
	const FRigPhysicsVisualizationSettings1& VisualizationSettings,
	const UWorld*                            DebugWorld) const
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_Draw);

	// Lock against access to the simulation, in case of access by the WorldObject task
	UE::TScopeLock<FTransactionallySafeCriticalSection> Lock(*SimulationMutex.Get());

#if CHAOS_DEBUG_DRAW
	// TODO enable/disable chaos debug draw with a cvar or flag
	if (DebugWorld && DebugWorld->GetPhysicsScene() && DebugWorld->GetPhysicsScene()->GetDebugDrawScene())
	{
		Simulation->SetDebugDrawScene(FString("ControlRig"), DebugWorld->GetPhysicsScene()->GetDebugDrawScene());
		Simulation->DebugDraw();
	}
#endif

	if (!DI)
	{
		return;
	}

	// All rendering is done relative to the component, so convert the sim space (identity) into the
	// component space.
	const FTransform SpaceTransform = ConvertSimSpaceTransformToComponentSpace(SolverSettings, FTransform());

	// Resolve each Show* toggle against its matching override CVar once per Draw call.
	const bool bShowBodies = GetControlRigPhysicsShow(
		VisualizationSettings.bShowBodies, CVarControlRigPhysicsShowBodiesOverride);
	const bool bShowCentreOfMass = GetControlRigPhysicsShow(
		VisualizationSettings.bShowCentreOfMass, CVarControlRigPhysicsShowCentreOfMassOverride);
	const bool bShowJoints = GetControlRigPhysicsShow(
		VisualizationSettings.bShowJoints, CVarControlRigPhysicsShowJointsOverride);
	const bool bShowControls = GetControlRigPhysicsShow(
		VisualizationSettings.bShowControls, CVarControlRigPhysicsShowControlsOverride);
	const bool bShowWorldObjects = GetControlRigPhysicsShow(
		VisualizationSettings.bShowWorldObjects, CVarControlRigPhysicsShowWorldObjectsOverride);
	const bool bShowWorldOverlapBox = GetControlRigPhysicsShow(
		VisualizationSettings.bShowWorldOverlapBox, CVarControlRigPhysicsShowWorldOverlapBoxOverride);
	const bool bShowActiveContacts = GetControlRigPhysicsShow(
		VisualizationSettings.bShowActiveContacts, CVarControlRigPhysicsShowActiveContactsOverride);
	const bool bShowInactiveContacts = GetControlRigPhysicsShow(
		VisualizationSettings.bShowInactiveContacts, CVarControlRigPhysicsShowInactiveContactsOverride);

	if (bShowBodies)
	{
		if (CollisionActorHandle)
		{
			DrawActor(*DI, VisualizationSettings, bShowCentreOfMass, SpaceTransform, *CollisionActorHandle);
		}

		for (const TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
		{
			const FRigBodyRecord& Record = BodyRecordPair.Value;
			if (const ImmediatePhysics::FActorHandle* ActorHandle = Record.ActorHandle)
			{
				DrawActor(*DI, VisualizationSettings, bShowCentreOfMass, SpaceTransform, *ActorHandle);
			}
		}
	}

	if (bShowJoints)
	{
		for (const TPair<FRigComponentKey, FRigJointRecord>& JointRecordPair : JointRecords)
		{
			const FRigJointRecord& Record = JointRecordPair.Value;
			if (const ImmediatePhysics::FJointHandle* JointHandle = Record.JointHandle)
			{
				DrawJoint(*DI, VisualizationSettings, SpaceTransform, *JointHandle, false);
			}
		}
	}

	if (bShowControls)
	{
		for (const TPair<FRigComponentKey, FRigControlRecord>& ControlRecordPair : ControlRecords)
		{
			const FRigControlRecord& Record = ControlRecordPair.Value;
			if (const ImmediatePhysics::FJointHandle* JointHandle = Record.JointHandle)
			{
				if (const Chaos::FPBDJointConstraintHandle* Constraint = JointHandle->GetConstraint())
				{
					if (IsConstraintDriveActive(*Constraint))
					{
						DrawJoint(*DI, VisualizationSettings, SpaceTransform, *JointHandle, true);
					}
				}
			}
		}
	}

	if (bShowWorldOverlapBox)
	{
		if (WorldOverlapBox.IsValid)
		{
			// Remember that rendering is done in component space
			FTransform BoxRelWorldTM(FQuat::Identity, WorldOverlapBox.GetCenter());
			FTransform BoxRelComponentTM = BoxRelWorldTM.GetRelativeTransform(SimulationSpaceState.ComponentTM);
			DI->DrawBox(
				BoxRelComponentTM,
				FTransform(FQuat::Identity, FVector::ZeroVector, WorldOverlapBox.GetSize()),
				FColor::Green,
				VisualizationSettings.LineThickness);
		}
	}

	if (bShowWorldObjects)
	{
		for (TPair<FWorldObjectKey, FWorldObject>& WorldObjectPair : *WorldObjects.Get())
		{
			FWorldObject& WorldObject = WorldObjectPair.Value;
			if (WorldObject.ActorHandle)
			{
				DrawActor(*DI, VisualizationSettings, bShowCentreOfMass, SpaceTransform, *WorldObject.ActorHandle);
			}
		}
	}

	if (bShowActiveContacts || bShowInactiveContacts)
	{
		Simulation->VisitCollisions(
			[DI, &SpaceTransform, &VisualizationSettings, bShowActiveContacts, bShowInactiveContacts]
			(const ImmediatePhysics::FSimulation::FCollisionData& Collision)
			{
				using namespace Chaos;

				bool bIsActive = Collision.GetCollisionAccumulatedImpulse().IsZero();
				if ((bIsActive && bShowActiveContacts) || (!bIsActive && bShowInactiveContacts))
				{
					float Size = 5.0f * VisualizationSettings.ShapeSize;

					int32 NumManifoldPoints = Collision.GetNumManifoldPoints();
					for (int32 PointIndex = 0; PointIndex != NumManifoldPoints; ++PointIndex)
					{
						Chaos::FRealSingle Depth;
						Chaos::FVec3 PlaneNormal;
						Chaos::FVec3 PointLocation;
						Chaos::FVec3 PlaneLocation;
						Collision.GetManifoldPointData(
							PointIndex, Depth, PlaneNormal, PointLocation, PlaneLocation);

						const FVec3 PointPlaneLocation = PointLocation -
							FVec3::DotProduct(PointLocation - PlaneLocation, PlaneNormal) * PlaneNormal;

						FMatrix Axes = FRotationMatrix::MakeFromZ(PlaneNormal);
						FTransform TM(Axes);
						TM.SetTranslation(PointPlaneLocation);

						DI->DrawCircle(
							SpaceTransform, TM, Size,
							bIsActive ? RigPhysicsSolverDraw::ActiveContactColor : RigPhysicsSolverDraw::InactiveContactColor,
							VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail);
					}
				}
			}, Chaos::ECollisionVisitorFlags::VisitDefault);
	}
}

