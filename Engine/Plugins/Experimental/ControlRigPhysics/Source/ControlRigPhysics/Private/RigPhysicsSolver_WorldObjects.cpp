// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigPhysicsSolver.h"
#include "RigPhysicsSolver_Space.inl"

#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsAdapters.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsSimulation.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsChaos/ImmediatePhysicsSimulation_Chaos.h"
#include "Chaos/ParticleHandle.h"

#include "Engine/OverlapResult.h"
#include "Engine/World.h"
#include "CollisionQueryParams.h"
#include "CollisionShape.h"

#include "Stats/Stats.h"
#include "Stats/StatsHierarchical.h"

#include "Components/InstancedStaticMeshComponent.h"

namespace 
{
	bool IsBodyDesiredInSim(const FBodyInstance* Body, bool bPhysicsCollision, bool bQueryCollision, bool bProbeCollision)
	{
		const ECollisionEnabled::Type CollisionEnabled = Body->GetCollisionEnabled();

		return (bPhysicsCollision && CollisionEnabledHasPhysics(CollisionEnabled))
			|| (bQueryCollision && CollisionEnabledHasQuery(CollisionEnabled))
			|| (bProbeCollision && CollisionEnabledHasProbe(CollisionEnabled));
	}
}

//======================================================================================================================
// Note that the simulation mutex will already be taken when we are called
void FRigPhysicsSolver::UpdateWorldObjectsPrePhysics(const FRigPhysicsSolverSettings& SolverSettings)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateWorldObjectsPrePhysics);

	if (SolverSettings.WorldCollisionType == ERigPhysicsWorldCollisionType::None)
	{
		for (TPair<FWorldObjectKey, FWorldObject>& WorldObjectPair : *WorldObjects.Get())
		{
			FWorldObject& WorldObject = WorldObjectPair.Value;
			if (WorldObject.ActorHandle)
			{
				UE_LOGF(LogRigPhysics, Warning, "Expiring world object %ls",
					*WorldObject.ActorHandle->GetName().ToString());

				Simulation->DestroyActor(WorldObject.ActorHandle);
			}
		}
		WorldObjects->Reset();
	}
	else
	{
		TArray<FWorldObjectKey> EntriesToRemove;
		for (TPair<FWorldObjectKey, FWorldObject>& WorldObjectPair : *WorldObjects.Get())
		{
			FWorldObject& WorldObject = WorldObjectPair.Value;
			if (WorldObject.GetExpired(UpdateCounter))
			{
				if (WorldObject.ActorHandle)
				{
					UE_LOGF(LogRigPhysics, Log, "Expiring world object %ls", 
						*WorldObject.ActorHandle->GetName().ToString());

					Simulation->DestroyActor(WorldObject.ActorHandle);
					WorldObject.ActorHandle = nullptr;
				}
				EntriesToRemove.Add(WorldObjectPair.Key);
			}
		}
		for (const FWorldObjectKey& EntryToRemove : EntriesToRemove)
		{
			WorldObjects->Remove(EntryToRemove);
		}
	}

	for (TPair<FWorldObjectKey, FWorldObject>& WorldObjectPair : *WorldObjects.Get())
	{
		FWorldObject& WorldObject = WorldObjectPair.Value;
		FTransform SimSpaceTM = ConvertWorldTransformToSimSpace(SolverSettings, WorldObject.ComponentWorldTransform);
		// Note that the code above should have ensured we don't have a null pointer, but let's be safe
		if (ensure(WorldObject.ActorHandle != nullptr))
		{
			if (WorldObject.LastSeenUpdateCounter == -1)
			{
				UE_LOGF(LogRigPhysics, Log,
					"Adding object %ls", *WorldObject.ActorHandle->GetName().ToString());

				// We need to call InitWorldTransform (not just SetWorldTransform) here, to ensure the
				// velocities are set to zero (though they should be after creation anyway), and that
				// the object has its bounds set.
				WorldObject.ActorHandle->InitWorldTransform(SimSpaceTM);
				// If the object is found in the next overlap, then its update counter will be updated.
				// However, in case it's not, we set it here too to make sure that it expires. 
				WorldObject.LastSeenUpdateCounter = UpdateCounter;
			}
			else
			{
				WorldObject.ActorHandle->SetKinematicTarget(SimSpaceTM);
			}
		}
	}
}

//======================================================================================================================
// Note that the simulation mutex will already be taken/locked when we are called 
// from FRigPhysicsSolver::StepSimulation
void FRigPhysicsSolver::UpdateWorldObjectsPostPhysics(
	const UWorld* World, const FRigPhysicsSolverSettings& SolverSettings, const AActor* OwningActorPtr)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigPhysics_UpdateWorldObjectsPostPhysics);

	WorldOverlapBox = FBox();
	if (SolverSettings.WorldCollisionType == ERigPhysicsWorldCollisionType::None)
	{
		return;
	}

	// Get the volume over which to do overlaps
	const FTransform SpaceTransform = 
		ConvertSimSpaceTransformToComponentSpace(SolverSettings, FTransform()) * SimulationSpaceState.ComponentTM;
	for (TPair<FRigComponentKey, FRigBodyRecord>& BodyRecordPair : BodyRecords)
	{
		const FRigComponentKey& ComponentKey = BodyRecordPair.Key;
		const FRigBodyRecord& Record = BodyRecordPair.Value;

		if (const ImmediatePhysics::FActorHandle* ActorHandle = Record.ActorHandle)
		{
			if (!ActorHandle->GetIsKinematic())
			{
				const Chaos::FGeometryParticleHandle* GeometryParticleHandle = ActorHandle->GetParticle();
				const FTransform ParticleTransform = GeometryParticleHandle->GetTransformXR() * SpaceTransform;

				const auto& AABB = GeometryParticleHandle->LocalBounds();
				if (!AABB.IsEmpty())
				{
					for (int32 Index = 0; Index != 8; ++Index)
					{
						WorldOverlapBox += ParticleTransform.TransformPosition(AABB.GetVertex(Index));
					}
				}
			}
		}
	}

	if (!WorldOverlapBox.IsValid)
	{
		return;
	}

	TWeakPtr<TMap<FWorldObjectKey, FWorldObject>> WeakWorldObjects = WorldObjects;
	int64 UpdateCounterCopy = UpdateCounter;
	const AActor* OwningActorPtrCopy = OwningActorPtr;
	TSharedPtr<ImmediatePhysics::FSimulation> SimulationCopy = Simulation;

	const TWeakObjectPtr<const UWorld> WeakWorld = World;

	FVector NewHalfExtent = WorldOverlapBox.GetExtent() * SolverSettings.WorldCollisionBoundsExpansion;
	FVector OverlapAABoxCenter = WorldOverlapBox.GetCenter();
	WorldOverlapBox = FBox(OverlapAABoxCenter - NewHalfExtent, OverlapAABoxCenter + NewHalfExtent);
	FCollisionShape OverlapAABox = FCollisionShape::MakeBox(NewHalfExtent);

	FCollisionQueryParams QueryParams = FCollisionQueryParams(
		SCENE_QUERY_STAT(ControlRigPhysicsFindGeometry), /*bTraceComplex=*/false);
	QueryParams.MobilityType = EQueryMobilityType::Any;

	FCollisionObjectQueryParams ObjectQueryParams;
	switch (SolverSettings.WorldCollisionType)
	{
	case ERigPhysicsWorldCollisionType::Static:
		ObjectQueryParams = FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllStaticObjects); break;
	case ERigPhysicsWorldCollisionType::Dynamic:
		ObjectQueryParams = FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllDynamicObjects); break;
	case ERigPhysicsWorldCollisionType::All:
		ObjectQueryParams = FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects); break;
	default:
		break;
	}

	// Query type data
	const bool bWorldCollisionIncPhysics = SolverSettings.bWorldCollisionIncludePhysics;
	const bool bWorldCollisionIncQuery = SolverSettings.bWorldCollisionIncludeQuery;
	const bool bWorldCollisionIncProbe = SolverSettings.bWorldCollisionIncludeProbe;

	// The actual overlap needs to be on the game thread, so we will run it in an async task.
	// However, note that between now and then, we/our simulation etc might get destroyed so we must
	// protect against that. Even the mutex guarding against access to the simulation may be released.
	if (ObjectQueryParams.GetObjectTypesToQuery())
	{
		// Make a weak copy of the simulation mutex and pass it to the (game thread) task. Note that
		// when we come to use it we need to check whether it it valid.
		TWeakPtr<FTransactionallySafeCriticalSection> WeakSimulationMutex = SimulationMutex;

		auto TaskBody =
			[WeakSimulationMutex, WeakWorld, WeakWorldObjects, OverlapAABox, OverlapAABoxCenter, QueryParams, 
			ObjectQueryParams, UpdateCounterCopy, OwningActorPtrCopy, SimulationCopy,
			bWorldCollisionIncPhysics, bWorldCollisionIncQuery, bWorldCollisionIncProbe]()
			{
				if (TSharedPtr<FTransactionallySafeCriticalSection> SimulationMutex = WeakSimulationMutex.Pin())
				{
					UE::TScopeLock<FTransactionallySafeCriticalSection> Lock(*SimulationMutex.Get());

					if (TStrongObjectPtr<const UWorld> World = WeakWorld.Pin())
					{
						if (TSharedPtr<TMap<FWorldObjectKey, FWorldObject>> WorldObjects = WeakWorldObjects.Pin())
						{
							TArray<FOverlapResult> Overlaps;
							World->OverlapMultiByObjectType(
								Overlaps, OverlapAABoxCenter, FQuat::Identity, ObjectQueryParams,
								OverlapAABox, QueryParams);

							for (const FOverlapResult& Overlap : Overlaps)
							{
								if (UPrimitiveComponent* OverlapComp = Overlap.GetComponent())
								{
									if (!IsValid(OverlapComp))
									{
										continue;
									}

									// For instanced static mesh components, each overlap result
									// corresponds to a specific instance via Overlap.ItemIndex.
									// The component-level BodyInstance has null BodySetup and no
									// physics actor, so we must use the per-instance body instead.
									int32 OverlapInstanceIndex = INDEX_NONE;
									if (OverlapComp->IsA<UInstancedStaticMeshComponent>())
									{
										OverlapInstanceIndex = Overlap.ItemIndex;
									}

									// Use a per-instance composite key for ISM so each instance
									// gets its own FWorldObject entry.
									FWorldObjectKey Key(OverlapComp->GetUniqueID(), OverlapInstanceIndex);

									FWorldObject* WorldObject = WorldObjects->Find(Key);
									if (!WorldObject)
									{
										if (!OverlapComp->GetOwner() || OverlapComp->GetOwner() == OwningActorPtrCopy)
										{
											continue;
										}

										const FBodyInstance* BodyInstanceToUse = OverlapComp->GetBodyInstance(NAME_None, false, OverlapInstanceIndex);
										if (BodyInstanceToUse == nullptr || !BodyInstanceToUse->IsValidBodyInstance())
										{
											continue;
										}

										// New object - add it to the sim if it has the desired collision settings
										if (IsBodyDesiredInSim(BodyInstanceToUse, bWorldCollisionIncPhysics, bWorldCollisionIncQuery, bWorldCollisionIncProbe))
										{
											// Chaos holds scale separately to the world transform of the body
											FTransform TransformToUse = BodyInstanceToUse->GetUnrealWorldTransform();
											TransformToUse.SetScale3D(BodyInstanceToUse->Scale3D);

											WorldObject = &WorldObjects->Add(Key, FWorldObject());

											WorldObject->WorldPrimitiveComponent = OverlapComp;
											WorldObject->InstanceBodyIndex = OverlapInstanceIndex;

											// Note that we need to create the simulation actor here, even
											// though we're in the game thread, so we can copy data out of the
											// object. This should be safe because we only run this task after
											// we've finished using the simulation in the worker thread.

											WorldObject->ActorHandle = SimulationCopy->CreateActor(
												ImmediatePhysics::MakeKinematicActorSetup(
													BodyInstanceToUse, TransformToUse));

											if (WorldObject->ActorHandle)
											{
#if WITH_EDITOR
												WorldObject->ActorHandle->SetName(*OverlapComp->GetOwner()->GetActorLabel());
#endif
												SimulationCopy->AddToCollidingPairs(WorldObject->ActorHandle);
											}
											WorldObject->ComponentWorldTransform = TransformToUse;

											// Flag that the simulation TM needs to be set, rather
											// than using a kinematic target.
											WorldObject->LastSeenUpdateCounter = -1;
										}
									}
									else
									{
										if (TStrongObjectPtr<UPrimitiveComponent> WPC =
											WorldObject->WorldPrimitiveComponent.Pin())
										{
											FBodyInstance* BodyInstanceToUse = WPC->GetBodyInstance(NAME_None, false, WorldObject->InstanceBodyIndex);

											if (BodyInstanceToUse == nullptr || !BodyInstanceToUse->IsValidBodyInstance())
											{
												// Instance has no valid physics body
												continue;
											}

											// Chaos holds scale separately to the world transform
											FTransform TransformToUse = BodyInstanceToUse->GetUnrealWorldTransform();
											TransformToUse.SetScale3D(BodyInstanceToUse->Scale3D);

											WorldObject->ComponentWorldTransform = TransformToUse;
											WorldObject->LastSeenUpdateCounter = UpdateCounterCopy;
										}
									}
								}
							}
						}
					}
				}
			};
#if WITH_EDITOR
			if (IsInGameThread())
			{
				TaskBody();
			}
			else
			{
				AsyncTask(ENamedThreads::GameThread, MoveTemp(TaskBody));
			}
#else 
			AsyncTask(ENamedThreads::GameThread, MoveTemp(TaskBody));
#endif
	}
}

