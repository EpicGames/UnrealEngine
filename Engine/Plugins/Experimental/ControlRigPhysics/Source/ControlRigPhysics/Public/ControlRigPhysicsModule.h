// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
DECLARE_LOG_CATEGORY_EXTERN(LogControlRigPhysics, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogControlRigPhysics, Log, All);
#endif

class FControlRigPhysicsModule : public IModuleInterface
{
public:

	// IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

