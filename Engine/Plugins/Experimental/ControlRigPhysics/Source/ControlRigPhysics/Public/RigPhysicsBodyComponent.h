// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigPhysicsData.h"

#include "RigPhysicsBodyComponent.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// A component that can be added to a joint/element that defines how a physical body can be "attached" to it.
// The body supports dynamic movement, collision, and a physics joint with this body's parent in the hierarchy.
USTRUCT(BlueprintType)
struct FRigPhysicsBodyComponent : public FRigBaseComponent
{
public:

	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigPhysicsBodyComponent)

	// Simulation/solver properties that should only be set during the control rig construction
	// event, before the body is instantiated
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsBodySolverSettings BodySolverSettings;

	// Dynamics properties that should only be set during the control rig construction event, before
	// the body is instantiated
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsDynamics Dynamics;

	// Collision properties that should only be set during the control rig construction event,
	// before the body is instantiated
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsCollision Collision;

	// Properties that can and will likely be modified at runtime
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics)
	FPhysicsControlModifierData BodyData;

	// The target for when this body is kinematic
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics)
	FTransform KinematicTarget;

	// A list of body components with which we should not collide. The solver component can also be
	// included to prevent collisions against its shapes. This is typically/must only be set during
	// the control rig construction event, before the body is instantiated
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics)
	TArray<FRigComponentKey> NoCollisionBodies;

	// Removes any existing collision, and replaces it with a shape calculated from the joint
	// positions (if possible). The shape will be a single box when there are enough joints, but
	// when there are only a few joints it may be a sphere or capsule, depending on what is most appropriate.
	//
	// @param Hierarchy The hierarchy - the joint positions will be used to calculate the shape.
	// @param MinAspectRatio The minimum box extent, as a proportion of the maximum box extent.
	// @param MinSize The minimum size (in each dimension) of the created shape.
	UE_API void AutoCalculateCollision(URigHierarchy* Hierarchy, float MinAspectRatio = 0.25f, float MinSize = 0.0f);

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

#if WITH_EDITOR
	UE_API virtual const FSlateIcon& GetIconForUI() const override;
#endif

	UE_API virtual void OnAddedToHierarchy(URigHierarchy* InHierarchy, URigHierarchyController* InController) override;

	UE_API virtual void OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }

	static FName GetDefaultName() { return TEXT("PhysicsBody"); }

	// TODO Should any/all of these (a) be UPROPERTYs, and thus visible (b) result in a call
	// to Hierarchy.Notify(...) when modified?

	// Forces (etc) that should be applied and then cleared at the next solver step.
	TArray<FPhysicsControlNamedForceAndTorqueData> ForceAndTorques;

	// Internal/runtime state, in component space
	FTransform Transform;
	FTransform CoMTransform;
	FVector    LinearVelocity = FVector::ZeroVector;
	FVector    AngularVelocity = FVector::ZeroVector;
};

#undef UE_API
