// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsExecution.h"

#include "RigPhysicsBodyExecution.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// Adds a new physics body as a component on the owner element.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Spawn Physics Body", Keywords = "Add,Construction,Create,New,Body,Skeleton", Varying))
struct FRigUnit_AddPhysicsBody : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddPhysicsBody()
	{
		Owner.Type = ERigElementType::Bone;
		Solver.PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		Solver.PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// The Physics Body component key that was created
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsBodyComponentKey;

	// The solver to relate this new physics body should be added to
	UPROPERTY(meta = (Input))
	FRigPhysicsBodySolverSettings Solver;

	// The dynamics properties of the new physics element	
	UPROPERTY(meta = (Input))
	FRigPhysicsDynamics Dynamics;

	// The collision properties of the new physics element
	UPROPERTY(meta = (Input))
	FRigPhysicsCollision Collision;

	// The runtime modifiable data
	UPROPERTY(meta = (Input))
	FPhysicsControlModifierData BodyData;
};

// Indicates whether the component exists and is a Physics Body
USTRUCT(meta=(DisplayName="Get Physics Body Exists", Keywords="Query,Body"))
struct FRigUnit_GetPhysicsBodyExists : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigPhysicsBodyComponent::GetDefaultName());

	// Whether the component exists and is a Physics Body
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

// Discards any existing collision data and replaces it with a shape based on the joint positions.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Calculate Physics Collision", Varying))
struct FRigUnit_HierarchyAutoCalculateCollision : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyAutoCalculateCollision()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// For boxes: The minimum box extent, as a proportion of the maximum box extent.
	// For capsules: The minimum radius, as a proportion of the length (not including the radius)
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float MinAspectRatio = 0.25f;

	// For boxes: The minimum side length. 
	// For capsules: The minimum radius
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float MinSize = 0.0f;
};

// Sets the body solver settings etc for a physics component body.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Set Physics Body Solver Settings", Varying))
struct FRigUnit_HierarchySetBodySolverSettings : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetBodySolverSettings()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Sets the body solver settings of the Physics Body
	UPROPERTY(meta = (Input))
	FRigPhysicsBodySolverSettings BodySolverSettings;
};


// Sets the mass etc for a physics component body.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Set Physics Body Dynamics Properties", Varying))
struct FRigUnit_HierarchySetDynamics : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetDynamics()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Sets the properties describing the dynamics properties of the Physics Body
	UPROPERTY(meta = (Input))
	FRigPhysicsDynamics Dynamics;
};

// Sets the collision for a physics component body.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Set Physics Body Collision Properties", Varying))
struct FRigUnit_HierarchySetCollision : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetCollision()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Specifies the collision associated with the Physics Body
	UPROPERTY(meta = (Input))
	FRigPhysicsCollision Collision;
};


// Disables collision between two bodies.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Disable Collision Between", Varying))
struct FRigUnit_HierarchyDisableCollisionBetween : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyDisableCollisionBetween()
	{
		PhysicsBodyComponentKey1.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey1.Name = FRigPhysicsBodyComponent::GetDefaultName();
		PhysicsBodyComponentKey2.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey2.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Component key for the first body
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey1;

	// Component key for the second body
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey2;
};

// Sets what bone is used as a source transform for the physics body. This is used as a kinematic target, and when
// initializing the simulation.
// Note: Although this can be changed after physics instantiation, doing so will produce a one-frame velocity
// spike because the source velocity is computed from the difference between the previous frame's source bone
// position and the new source bone position, which may produce undesirable simulation behaviour (including
// tripping the auto-reset thresholds).
USTRUCT(meta = (DisplayName = "Set Physics Body Source Bone", Varying))
struct FRigUnit_HierarchySetPhysicsBodySourceBone : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodySourceBone()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		SourceBone.Type = ERigElementType::Bone;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The bone to use as a source transform (kinematic target, and initialization) for the Physics Body.
	UPROPERTY(meta = (Input))
	FRigElementKey SourceBone;
};

// Sets what bone is targeted by the simulation - i.e. where the simulation output is written to.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Set Physics Body Target Bone", Varying))
struct FRigUnit_HierarchySetPhysicsBodyTargetBone : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyTargetBone()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		TargetBone.Type = ERigElementType::Bone;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The bone to use as a target for the Physics Body - i.e. where its transforms should be written to.
	UPROPERTY(meta = (Input))
	FRigElementKey TargetBone;
};

// Sets all the data on a body - but in a sparse way so you can decide which parameters get applied.
// Note that the sparse data does not get displayed correctly (the flags that enable/disable all end up getting
// reset if the user attempts to change them) so this should be used with caution until this is fixed.
USTRUCT(meta = (DisplayName = "Set Physics Body Data", Varying))
struct FRigUnit_HierarchySetPhysicsBodySparseData : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodySparseData()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Sparse data to be set on the Physics Body
	UPROPERTY(meta = (Input))
	FPhysicsControlModifierSparseData Data;
};

// Sets the kinematic target for a body - note that this won't actually make the body kinematic
USTRUCT(meta = (DisplayName = "Set Physics Body Kinematic Target", Varying))
struct FRigUnit_HierarchySetPhysicsBodyKinematicTarget : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyKinematicTarget()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Kinematic target for the Physics Body. This will be combined with the animation pose,
	// depending on the Kinematic Target Space.
	UPROPERTY(meta = (Input))
	FTransform KinematicTarget;
};

// Sets the kinematic target space for a body - note that this won't actually make the body kinematic
USTRUCT(meta = (DisplayName = "Set Physics Body Kinematic Target Space", Varying))
struct FRigUnit_HierarchySetPhysicsBodyKinematicTargetSpace : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyKinematicTargetSpace()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The space for kinematic targets.
	UPROPERTY(meta = (Input))
	EPhysicsControlKinematicTargetSpace KinematicTargetSpace = EPhysicsControlKinematicTargetSpace::OffsetInBoneSpace;
};

// Sets the movement mode for this body.
USTRUCT(meta = (DisplayName = "Set Physics Body Movement Mode", Varying))
struct FRigUnit_HierarchySetPhysicsBodyMovementType : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyMovementType()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// How the Physics Body should move when the Physics Solver is stepped.
	UPROPERTY(meta = (Input))
	EPhysicsMovementType MovementType = EPhysicsMovementType::Simulated;
};

// Sets what collision mode is used for this body
USTRUCT(meta = (DisplayName = "Set Physics Body Collision Mode", Varying))
struct FRigUnit_HierarchySetPhysicsBodyCollisionType : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyCollisionType()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// What type of collision to use for the Physics Body
	UPROPERTY(meta = (Input))
	TEnumAsByte<ECollisionEnabled::Type> CollisionType = ECollisionEnabled::QueryAndPhysics;
};

// Sets whether this body should be included in checks for resetting physics on the whole rig.
// Note: The flag is read by the solver each step, so this can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Body Include In Checks For Reset", Varying))
struct FRigUnit_HierarchySetPhysicsBodyIncludeInChecksForReset : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyIncludeInChecksForReset()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Whether the Physics Body should be included when checking to see if the simulation should be reset.
	UPROPERTY(meta = (Input))
	bool bInclude = true;
};

// Applies the material settings to all the collision shapes.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Set Physics Body Material", Varying))
struct FRigUnit_HierarchySetPhysicsBodyMaterial : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyMaterial()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The physics material to be used for collision on the Physics Body
	UPROPERTY(meta = (Input))
	FRigPhysicsMaterial Material;
};

// Sets the multiplier on gravity that should be applied to the body.
USTRUCT(meta = (DisplayName = "Set Physics Body Gravity Multiplier", Varying))
struct FRigUnit_HierarchySetPhysicsBodyGravityMultiplier : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyGravityMultiplier()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// How much the Physics Body should respond to the gravity set in the Physics Solver
	UPROPERTY(meta = (Input))
	float GravityMultiplier = 1.0f;
};

// Controls the amount that the simulation is blended back into the target bones.
USTRUCT(meta = (DisplayName = "Set Physics Body Physics Blend Weight", Varying))
struct FRigUnit_HierarchySetPhysicsBodyPhysicsBlendWeight : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyPhysicsBlendWeight()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// A per-body "alpha" on how much of the physics movement to apply to the target bone. The
	// effective blend for this body is Alpha * PhysicsBlendWeight, applied at hierarchy
	// writeback. A weight of zero leaves the bone at the input animation pose, but the body
	// still simulates (and still affects neighbouring bodies through joints/contacts). Only the
	// Step Physics Solver's Alpha pin triggers the cheap simulation-bypass pass-through. Setting
	// every body's PhysicsBlendWeight to zero does not.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float PhysicsBlendWeight = 1.0f;
};

// If true, then kinematic objects will be written back from simulation to the bones. This only
// necessary when either kinematic targets are being used, or when the target bone differs from the source bone.
USTRUCT(meta = (DisplayName = "Set Physics Body Update Kinematic From Simulation", Varying))
struct FRigUnit_HierarchySetPhysicsBodyUpdateKinematicFromSimulation : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyUpdateKinematicFromSimulation()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// When bodies are written back from simulation, if this is set to false, there is a small
	// performance benefit to skipping bodies that are known to be kinematic, and when they are set
	// to just track the input animation.
	UPROPERTY(meta = (Input))
	bool bUpdateKinematicFromSimulation = true;
};

// Sets the linear and angular damping on the body. This will reduce the velocity, to make the
// body start tracking the simulation space.
// Note: Although damping is currently stored on Dynamics, the solver re-reads it each step, so this
// can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Body Damping", Varying))
struct FRigUnit_HierarchySetPhysicsBodyDamping : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyDamping()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The desired linear damping
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LinearDamping = 0;

	// The desired angular damping
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularDamping = 0;
};

// Enable/disable CCD. Note that this can be expensive, so disable it when not needed!
// Requires CCD to be allowed in the solver.
USTRUCT(meta = (DisplayName = "Set Physics Body Enable CCD", Varying))
struct FRigUnit_HierarchySetPhysicsBodyEnableCCD : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsBodyEnableCCD()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// Enable / disable CCD. Note that this can be expensive, so disable it when not needed!
	// Requires CCD to be allowed in the solver.
	UPROPERTY(meta = (Input))
	bool bEnableCCD = false;
};


// This will add a force (or impulse, torque, velocity change etc) to the body. The force record
// will be stored in the physics body component, and then applied next time the solver is stepped,
// after which the record will be cleared.
USTRUCT(meta = (DisplayName = "Add Force", Varying, Keywords = "Physics, Impulse, Torque, Acceleration, Velocity"))
struct FRigUnit_HierarchyAddPhysicsBodyForce : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyAddPhysicsBodyForce()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The desired force/torque to be applied
	UPROPERTY(meta = (Input))
	FPhysicsControlForceAndTorqueData ForceAndTorque;

	// The optional name of this force record. Used to identify the force records, and potentially delete it.
	// Can be blank.
	UPROPERTY(meta = (Input))
	FName Name;
};

// Removes all force records (assuming they hasn't been applied yet) with the given Name on the Body component.
USTRUCT(meta = (DisplayName = "Remove Force", Varying, Keywords = "Physics, Impulse, Torque, Acceleration, Velocity"))
struct FRigUnit_HierarchyRemovePhysicsBodyForce : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyRemovePhysicsBodyForce()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The name of the force record to remove.
	UPROPERTY(meta = (Input))
	FName Name;
};

// Retrieves the current simulation transform of the body.
USTRUCT(meta = (DisplayName = "Get Physics Body Transform", Varying))
struct FRigUnit_HierarchyGetPhysicsBodyTransform : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPhysicsBodyTransform()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be queried
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The current body transform.
	UPROPERTY(meta = (Output))
	FTransform Transform;
};

// Retrieves the current simulation centre of mass transform of the body.
USTRUCT(meta = (DisplayName = "Get Physics Body CoM Transform", Varying))
struct FRigUnit_HierarchyGetPhysicsBodyCoMTransform : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPhysicsBodyCoMTransform()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be queried
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The current centre of mass transform of the body.
	UPROPERTY(meta = (Output))
	FTransform CoMTransform;
};

// Retrieves the current centre of mass linear velocity of the body.
USTRUCT(meta = (DisplayName = "Get Physics Body Linear Velocity", Varying))
struct FRigUnit_HierarchyGetPhysicsBodyLinearVelocity : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPhysicsBodyLinearVelocity()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be queried
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The current body linear velocity.
	UPROPERTY(meta = (Output))
	FVector LinearVelocity = FVector::ZeroVector;
};

// Retrieves the current centre of mass angular velocity of the body.
USTRUCT(meta = (DisplayName = "Get Physics Body Angular Velocity", Varying))
struct FRigUnit_HierarchyGetPhysicsBodyAngularVelocity : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPhysicsBodyAngularVelocity()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be queried
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The current body angular velocity.
	UPROPERTY(meta = (Output))
	FVector AngularVelocity = FVector::ZeroVector;
};

// Retrieves the current linear velocity of the body at a point in space.
USTRUCT(meta = (DisplayName = "Get Physics Body Point Velocity", Varying))
struct FRigUnit_HierarchyGetPhysicsBodyPointVelocity : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetPhysicsBodyPointVelocity()
	{
		PhysicsBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Body to be queried
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsBodyComponentKey;

	// The position to calculate the velocity at.
	UPROPERTY(meta = (Input))
	FVector Position = FVector::ZeroVector;

	// The space that Position is specified in.
	UPROPERTY(meta = (Input))
	EPhysicsControlSpace Space = EPhysicsControlSpace::Body;

	// The current velocity at Position.
	UPROPERTY(meta = (Output))
	FVector Velocity = FVector::ZeroVector;
};


#undef UE_API
