// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsControlData.h"
#include "RigPhysicsBodyComponent.h"

#include "RigPhysicsControlComponent.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

enum class ERigPhysicsControlComponentDirtyFlags : uint8
{
	None = 0,
	Data = 1 << 0, // Strength etc (not target)
	Target = 1 << 1,
	All = Data | Target
};
ENUM_CLASS_FLAGS(ERigPhysicsControlComponentDirtyFlags);


// A component that can be added to hierarchy elements (joints) to add the data required to control
// the simulation of them
USTRUCT(BlueprintType)
struct FRigPhysicsControlComponent : public FRigBaseComponent
{
public:

	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigPhysicsControlComponent)

	FRigPhysicsControlComponent()
	{
		ParentBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ParentBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		ChildBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ChildBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	// The body that controls the body being controlled. If this is dynamic, it will be affected
	// too. If unset, then it implies a global control.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	FRigComponentKey ParentBodyComponentKey;

	// If true, and if the parent body component key is not set, the parent body takes a default
	// value, obtained by searching upwards in the hierarchy until a bone with a suitable physics
	// body is found (if there is no body found, then the control will be inactive). If it is false,
	// then this search is not done, so the control will be in simulation space.
	UPROPERTY(BlueprintReadOnly, DisplayName = "Use Parent Body", EditAnywhere, Category = Control)
	bool bUseParentBodyAsDefault = false;

	// The body being controlled
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	FRigComponentKey ChildBodyComponentKey;

	// Describes the initial strength etc of the new control
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	FPhysicsControlData ControlData;

	// This is the currently active control multiplier. 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	FPhysicsControlMultiplier ControlMultiplier;

	// Describes the initial target for the new control
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Control)
	FPhysicsControlTarget ControlTarget;

	// Flags indicating what has been changed and needs to be set in the physics representation
	mutable ERigPhysicsControlComponentDirtyFlags DirtyFlags = ERigPhysicsControlComponentDirtyFlags::All;

	UE_API virtual void Save(FArchive& A) override;
	UE_API virtual void Load(FArchive& Ar) override;

	UE_API virtual bool CanBeAddedTo(
		const FRigElementKey& InElementKey, const URigHierarchy* InHierarchy, FString* OutFailureReason) const override;

	UE_API virtual void OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }

	static FName GetDefaultName() { return TEXT("PhysicsControl"); }
};

#undef UE_API
