// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsExecution.h"
#include "RigPhysicsControlComponent.h"

#include "RigPhysicsControlExecution.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// Adds a new physics control as a component on the owner element.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (Category = "RigPhysics", NodeColor = "1.0 0.6 0.3", DisplayName = "Spawn Physics Control", Keywords = "Add,Construction,Create,New,Control", Varying))
struct FRigUnit_AddPhysicsControl : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddPhysicsControl()
	{
		Owner.Type = ERigElementType::Bone;
		ParentBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ParentBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		ChildBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ChildBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// The Physics Control component key that was created
	UPROPERTY(meta = (Output))
	FRigComponentKey ControlComponentKey;

	// The optional body that "does" the controlling - though if it is dynamic then it can move too
	UPROPERTY(meta = (Input))
	FRigComponentKey ParentBodyComponentKey;

	// Whether or not to use the parent body (if one exists) as the Physics Control parent
	UPROPERTY(meta = (Input), DisplayName = "Use Parent Body")
	bool bUseParentBodyAsDefault = false;

	// The body that is controlled
	UPROPERTY(meta = (Input))
	FRigComponentKey ChildBodyComponentKey;

	// Describes the initial strength etc of the new control
	UPROPERTY(meta = (Input))
	FPhysicsControlData ControlData;

	// Fine control over the control strengths etc
	UPROPERTY(meta = (Input))
	FPhysicsControlMultiplier ControlMultiplier;

	// The initial target for the new control
	UPROPERTY(meta = (Input))
	FPhysicsControlTarget ControlTarget;

	// TODO Which set to include the control in (optional). Note that it automatically gets added to the set "All"
	//UPROPERTY(meta = (Input))
	//FName Set;
};

// Indicates whether the component exists and is a Physics Control
USTRUCT(meta=(DisplayName="Get Physics Control Exists", Keywords="Query,Control"))
struct FRigUnit_GetPhysicsControlExists : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigPhysicsControlComponent::GetDefaultName());

	// Whether the component exists and is a Physics Control
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

// Sets whether a control is enabled
USTRUCT(meta = (DisplayName = "Set Physics Control Enabled", Varying))
struct FRigUnit_HierarchySetControlEnabled : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlEnabled()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control that should be enabled or disabled
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// Whether or not the Physics Control should be enabled
	UPROPERTY(meta = (Input))
	bool bEnabled = true;
};

// Sets the custom control point on a control
USTRUCT(meta = (DisplayName = "Set Physics Control Custom Control Point", Varying))
struct FRigUnit_HierarchySetControlCustomControlPoint : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlCustomControlPoint()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The position of the control point relative to the child mesh, when using a custom control point.
	UPROPERTY(meta = (Input))
	FVector CustomControlPoint = FVector::ZeroVector;

	// Whether or not to use the custom control point position
	UPROPERTY(meta = (Input))
	bool bUseCustomControlPoint = true;
};


// Sets the control data for a physics control
USTRUCT(meta = (DisplayName = "Set Physics Control Data", Varying))
struct FRigUnit_HierarchySetControlData : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlData()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The data (strengths etc) that should be set on the Physics Control
	UPROPERTY(meta = (Input))
	FPhysicsControlData ControlData;
};

// Gets the control data for a physics control
USTRUCT(meta = (DisplayName = "Get Physics Control Data", Varying))
struct FRigUnit_HierarchyGetControlData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetControlData()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
		UE_API virtual void Execute() override;

	// The Physics Control component from which to retrieve data
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The current control data (strengths etc) 
	UPROPERTY(meta = (Output))
	FPhysicsControlData ControlData;
};

// Sets the Linear Strength of a Physics Control
USTRUCT(meta = (DisplayName = "Set Physics Control Linear Strength", Varying))
struct FRigUnit_HierarchySetControlLinearStrength : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlLinearStrength()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The linear strength of the control. In the absence of any other influences, this is the
	// frequency of oscillation when there is zero damping. For example, a strength of 2 would make
	// the target oscillate twice per second (when there is no damping). Note that parent-space
	// controls should this set to zero when they are controlling two bodies that are connected with
	// a physics joint.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float Strength = 0.0f;
};

// Sets the Linear Damping Ratio of a Physics Control
USTRUCT(meta = (DisplayName = "Set Physics Control Linear Damping Ratio", Varying))
struct FRigUnit_HierarchySetControlLinearDampingRatio : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlLinearDampingRatio()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The linear damping ratio for the control. When this is 1, there is just enough damping to
	// stop the control from oscillating about its target, due to the control strength (in the
	// absence of any other influences). Values above this will add more damping, and values below
	// will make the system tend to oscillate, as it will be under-ramped.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 1.0f;
};

// Sets the Angular Strength of a Physics Control
USTRUCT(meta = (DisplayName = "Set Physics Control Angular Strength", Varying))
struct FRigUnit_HierarchySetControlAngularStrength : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlAngularStrength()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The angular strength of the control. In the absence of any other influences, this is the
	// frequency of oscillation when there is zero damping. For example, a strength of 2 would make
	// the target oscillate twice per second (when there is no damping).
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float Strength = 0.0f;
};

// Sets the Angular Damping Ratio of a Physics Control
USTRUCT(meta = (DisplayName = "Set Physics Control Angular Damping Ratio", Varying))
struct FRigUnit_HierarchySetControlAngularDampingRatio : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlAngularDampingRatio()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The angular damping ratio for the control. When this is 1, there is just enough damping to
	// stop the control from oscillating about its target, due to the control strength (in the
	// absence of any other influences). Values above this will add more damping, and values below
	// will make the system tend to oscillate, as it will be under-ramped.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 1.0f;
};

// Sets the Target velocity multipliers of a Physics Control
USTRUCT(meta = (DisplayName = "Set Physics Control Target Velocity Multipliers", Varying))
struct FRigUnit_HierarchySetControlTargetVelocityMultipliers : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlTargetVelocityMultipliers()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The multiplier for the linear target velocity. Set this to zero to see sim-space damping. Set
	// it to one to track the targets more closely.
	UPROPERTY(meta = (Input))
	float LinearTargetVelocityMultiplier = 1.0f;

	// The multiplier for the angular target velocity. Set this to zero to see sim-space damping. Set
	// it to one to track the targets more closely.
	UPROPERTY(meta = (Input))
	float AngularTargetVelocityMultiplier = 1.0f;
};

// Sets the multipliers for a physics control
USTRUCT(meta = (DisplayName = "Set Physics Control Multiplier", Varying))
struct FRigUnit_HierarchySetControlMultiplier : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlMultiplier()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The multipliers to use on the control. These allow fine-grained control over the strengths
	// (etc) in different directions.
	UPROPERTY(meta = (Input))
	FPhysicsControlMultiplier ControlMultiplier;
};

// Sets the control data and multiplier for a physics control
USTRUCT(meta = (DisplayName = "Set Physics Control Data And Multiplier", Varying))
struct FRigUnit_HierarchySetControlDataAndMultiplier : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlDataAndMultiplier()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The control data (strengths etc) to use
	UPROPERTY(meta = (Input))
	FPhysicsControlData ControlData;

	// Detail/per-direction multipliers on the control data
	UPROPERTY(meta = (Input))
	FPhysicsControlMultiplier ControlMultiplier;
};

// Sets the target for a physics control
USTRUCT(meta = (DisplayName = "Set Physics Control Target", Varying))
struct FRigUnit_HierarchySetControlTarget : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetControlTarget()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The target transform (etc) for the control
	UPROPERTY(meta = (Input))
	FPhysicsControlTarget ControlTarget;
};

// Sets the target for a physics control and updates the target velocities based on the previous
// targets (which will be overwritten)
USTRUCT(meta = (DisplayName = "Update Physics Control Target", Varying))
struct FRigUnit_HierarchyUpdateControlTarget : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyUpdateControlTarget()
	{
		PhysicsControlComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsControlComponentKey.Name = FRigPhysicsControlComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Control component to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsControlComponentKey;

	// The target position of the child body, relative to the parent body
	UPROPERTY(meta = (Input))
	FVector TargetPosition = FVector::ZeroVector;

	// The target orientation of the child body, relative to the parent body
	UPROPERTY(meta = (Input))
	FRotator TargetOrientation = FRotator::ZeroRotator;

	// The delta time used to calculate the target velocity
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DeltaTime = 0.0f;
};


#undef UE_API
