// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsControlData.h"

#include "Rigs/RigHierarchyDefines.h"
#include "Rigs/RigHierarchyComponents.h"

#include "PhysicsEngine/ConstraintTypes.h"
#include "PhysicsEngine/ConstraintDrives.h"

#include "RigPhysicsData.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

struct FRigVMExecuteContext;
struct FRigBaseElement;
struct FRigControlElement;
class URigHierarchy;
class FRigElementKeyRedirector;

const float RigPhysicsDefaultShapeSize = 10.0f;

UE_API FArchive& operator <<(FArchive& Ar, FConstraintDrive& Data);
UE_API FArchive& operator <<(FArchive& Ar, FLinearDriveConstraint& Data);
UE_API FArchive& operator <<(FArchive& Ar, FAngularDriveConstraint& Data);

// Legacy visualization toggles for the rig's physics simulation. Enabling any bShow* flag activates
// the low-level Chaos debug draw - which has significant runtime cost - so disable everything
// before profiling.
USTRUCT(BlueprintType)
struct FRigPhysicsVisualizationSettings
{
	GENERATED_BODY()

	// Legacy master enable (has been replaced by FRigPhysicsVisualizationSettings1). Preserved so
	// that nodes in existing assets keep their pin connections. Runtime gating is now done via CVar
	// ControlRig.Physics.AllowVisualization combined with the per-element bShow* flags on 
	// FRigPhysicsVisualizationSettings1.
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bEnableVisualization = false;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	float LineThickness = 1.0f;

	// Multiplier on the size things like limit shapes
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	int32 ShapeSize = 1;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	int32 ShapeDetail = 16;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowBodies = true;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowCentreOfMass = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowJoints = true;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowControls = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowWorldObjects = true;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowWorldOverlapBox = false;

	// If visualization is enabled, show active contacts. Note that this can be overridden using
	// CVar ControlRig.Physics.ShowActiveContactsOverride
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowActiveContacts = true;

	// If visualization is enabled, show inactive contacts. Note that this can be overridden using
	// CVar ControlRig.Physics.ShowInactiveContactsOverride
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowInactiveContacts = false;
};

// Visualization toggles for the rig's physics simulation. Enabling any bShow* flag activates the
// low-level Chaos debug draw - which has significant runtime cost - so disable everything before
// profiling. Visualization can also be globally suppressed at runtime via the CVar
// ControlRig.Physics.AllowVisualization (set to 0).
USTRUCT(BlueprintType)
struct FRigPhysicsVisualizationSettings1
{
	GENERATED_BODY()

	FRigPhysicsVisualizationSettings1() = default;

	// Does an "upgrade" from the old structure
	FRigPhysicsVisualizationSettings1(const FRigPhysicsVisualizationSettings& Other);

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	float LineThickness = 1.0f;

	// Multiplier on the size things like limit shapes
	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	int32 ShapeSize = 1;

	UPROPERTY(EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	int32 ShapeDetail = 16;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowBodies = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowCentreOfMass = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowJoints = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowControls = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowWorldObjects = false;

	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowWorldOverlapBox = false;

	// If visualization is enabled, show active contacts. Note that this can be overridden using
	// CVar ControlRig.Physics.ShowActiveContactsOverride
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowActiveContacts = false;

	// If visualization is enabled, show inactive contacts. Note that this can be overridden using
	// CVar ControlRig.Physics.ShowInactiveContactsOverride
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bShowInactiveContacts = false;
};

// Specifies whether/how simulation bodies should collide with the world
UENUM()
enum class ERigPhysicsWorldCollisionType : uint8
{
	// World collision is disabled
	None,
	// Control rig simulation bodies will only collide against static (or kinematic, since they may
	// move) objects in the world
	Static,
	// Control rig simulation bodies will only collide against dynamic/simulated objects in the world
	Dynamic,
	// Control rig simulation bodies will collide against all objects in the world
	All,
};

// Specifies what space the simulation should run in
UENUM()
enum class ERigPhysicsSimulationSpace : uint8
{
	// Simulate in world space. Moving the skeletal mesh will generate velocity changes
	World,
	// Simulate in component space. Moving the entire skeletal mesh will have no affect on velocities
	// unless this is explicitly enabled in the simulation space settings
	Component,
	// Simulate in the space of the specified bone. Moving the entire skeletal mesh and/or the Bone
	// will have no affect on velocities unless this is explicitly enabled in the simulation space settings.
	SpaceBone,
};

// Properties common to all collision shape types
USTRUCT(BlueprintType)
struct FRigPhysicsCollisionShape
{
	GENERATED_BODY()

	// Offset used when generating contact points. This allows you to smooth out the Minkowski sum
	// by radius R. Useful for making objects slide smoothly on top of irregularities
	UPROPERTY(Category = Shape, EditAnywhere, meta = (ClampMin = "0.0"))
	float RestOffset = 0.0f;

	UPROPERTY(Category = Shape, EditAnywhere)
	FName Name;

	// True if this shape should contribute to the overall mass of the body it belongs to. This lets
	// you create extra ollision volumes which do not affect the mass properties of an object.
	UPROPERTY(Category = Shape, EditAnywhere)
	bool bContributeToMass = true;

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionShape& Data);
};

// Box collision
USTRUCT(BlueprintType)
struct FRigPhysicsCollisionBox : public FRigPhysicsCollisionShape
{
	GENERATED_BODY()

	FRigPhysicsCollisionBox(const FTransform& InTM, const FVector& InExtents) : 
		TM(InTM), Extents(InExtents) {}
	FRigPhysicsCollisionBox() {}

	UPROPERTY(EditAnywhere, Category = Shapes)
	FTransform TM;

	// These are the full extents of the box in each axis
	UPROPERTY(EditAnywhere, Category = Shapes)
	FVector Extents = FVector::OneVector * RigPhysicsDefaultShapeSize;

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionBox& Data);
};

// Sphere collision
USTRUCT(BlueprintType)
struct FRigPhysicsCollisionSphere : public FRigPhysicsCollisionShape
{
	GENERATED_BODY()

	FRigPhysicsCollisionSphere(const FTransform& InTM, const float InRadius) :
		TM(InTM), Radius(InRadius) {
	}
	FRigPhysicsCollisionSphere() {}

	UPROPERTY(EditAnywhere, Category = Shapes)
	FTransform TM;

	UPROPERTY(EditAnywhere, Category = Shapes, meta = (ClampMin = "0.0"))
	float Radius = RigPhysicsDefaultShapeSize;

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionSphere& Data);
};

// Capsule collision
USTRUCT(BlueprintType)
struct FRigPhysicsCollisionCapsule : public FRigPhysicsCollisionShape
{
	GENERATED_BODY()

	FRigPhysicsCollisionCapsule(const FTransform& InTM, const float InRadius, const float InLength)
		: TM(InTM), Radius(InRadius), Length(InLength) {}
	FRigPhysicsCollisionCapsule() {}

	UPROPERTY(EditAnywhere, Category = Shapes)
	FTransform TM;

	UPROPERTY(EditAnywhere, Category = Shapes, meta = (ClampMin = "0.0"))
	float Radius = RigPhysicsDefaultShapeSize;

	// This is the length of the core part of the capsule. The total length will be Length + 2 * Radius
	UPROPERTY(EditAnywhere, Category = Shapes, meta = (ClampMin = "0.0"))
	float Length = RigPhysicsDefaultShapeSize;

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionCapsule& Data);
};

// Convex hull collision. VertexData holds the hull points in the element's local space
USTRUCT(BlueprintType)
struct FRigPhysicsCollisionConvex : public FRigPhysicsCollisionShape
{
	GENERATED_BODY()

	FRigPhysicsCollisionConvex(const FTransform& InTM, const TArray<FVector>& InVertexData)
		: TM(InTM), VertexData(InVertexData) {}
	FRigPhysicsCollisionConvex() {}

	UPROPERTY(EditAnywhere, Category = Shapes)
	FTransform TM;

	// Hull vertices, in the element's local space.
	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<FVector> VertexData;

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollisionConvex& Data);
};

// How to combine friction/restitution values. 
UENUM(BlueprintType)
enum class ERigPhysicsCombineMode : uint8
{
	Avg,
	Min,
	Multiply,
	Max
	// The values here must match those in Chaos::FChaosPhysicsMaterial::ECombineMode
};

// Material properties of a collision shapes.
USTRUCT(BlueprintType)
struct FRigPhysicsMaterial
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsMaterial& Data);

	// Friction is simplified here - just one value used for static and dynamic friction. 
	UPROPERTY(EditAnywhere, Category = Material, meta = (ClampMin = "0.0"))
	float Friction = 1.0f;

	UPROPERTY(EditAnywhere, Category = Material, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Restitution = 0.0f;

	// How to combine friction values. If the materials have different combine modes, then Max is
	// used over Multiply, Multiply over Min and Min over Avg.
	UPROPERTY(EditAnywhere, Category = Material)
	ERigPhysicsCombineMode FrictionCombineMode = ERigPhysicsCombineMode::Multiply;

	// How to combine restitution values. If the materials have different combine modes, then Max is
	// used over Multiply, Multiply over Min and Min over Avg.
	UPROPERTY(EditAnywhere, Category = Material)
	ERigPhysicsCombineMode RestitutionCombineMode = ERigPhysicsCombineMode::Multiply;
};

// Collection of shapes that define the collision and (optionally) mass distribution of the body
USTRUCT(BlueprintType)
struct FRigPhysicsCollision
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsCollision& Data);

	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<FRigPhysicsCollisionBox> Boxes;

	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<FRigPhysicsCollisionSphere> Spheres;

	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<FRigPhysicsCollisionCapsule> Capsules;

	UPROPERTY(EditAnywhere, Category = Shapes)
	TArray<FRigPhysicsCollisionConvex> Convexes;

	UPROPERTY(EditAnywhere, Category = Shapes)
	FRigPhysicsMaterial Material;

	bool IsEmpty() const { return Boxes.IsEmpty() && Spheres.IsEmpty() && Capsules.IsEmpty() && Convexes.IsEmpty(); }
	int32 NumShapes() const { return Boxes.Num() + Capsules.Num() + Spheres.Num() + Convexes.Num(); }
};

// Legacy bundled simulation-space settings. Retained as the input/output pin shape for the
// deprecated FRigUnit_AddPhysicsSolver, FRigUnit_SetPhysicsSolverSimulationSpaceSettings, and
// FRigUnit_GetPhysicsSolverSimulationSpaceSettings rig units only. New code should use
// FRigPhysicsSimulationSpaceMotion + FRigPhysicsTeleportDetectionSettings on
// FRigPhysicsSolverComponent instead.
USTRUCT(BlueprintType)
struct FRigPhysicsSimulationSpaceSettings
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationSpaceSettings& Data);

	// Overall multiplier on the effects of simulation space movement on the simulation
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float SpaceMovementAmount = 1.0f;

	// Multiplier on the components of vertical velocity of the simulation space that is passed to the simulation.
	// Usually from 0.0 to 1.0 to reduce the effects of jumping and crouching on the simulation, but
	// it can be higher than 1.0 if you need to exaggerate this motion for some reason.
	UPROPERTY(EditAnywhere, Category = Settings)
	float VelocityScaleZ = 1.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearVelocity = false;

	// A clamp on the effective world-space velocity that is passed to the simulation. The default
	// value effectively means "unlimited". It is not usually required to change this but you would
	// reduce this to limit the effects of drag on the bodies in the simulation (if you have bodies
	// that have LinearDrag set to non-zero in the physics asset). Expected values in this case
	// would be somewhat less than the usual velocities of your object which is commonly a few
	// hundred for a character.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearVelocity"))
	float MaxLinearVelocity = 10000;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularVelocity = false;

	// A clamp on the effective world-space angular velocity that is passed to the simulation. Units
	// are radian/s, so a value of about 6.0 is one rotation per second. The default value
	// effectively means "unlimited". You would reduce this (and MaxAngularAcceleration) to limit
	// how much bodies "fly out" when the actor spins on the spot. This is especially useful if you
	// have characters than can rotate very quickly and you would probably want values around or
	// less than 10 in this case.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularVelocity"))
	float MaxAngularVelocity = 10000;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearAcceleration = false;

	// A clamp on the effective world-space acceleration that is passed to the simulation. The
	// default value effectively means "unlimited". This property is used to stop the bodies of the
	// simulation flying out when suddenly changing linear speed. It is useful when you have
	// characters than can changes from stationary to running very quickly such as in an FPS. A
	// common value for a character might be in the few hundreds.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearAcceleration"))
	float MaxLinearAcceleration = 10000;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularAcceleration = false;

	// A clamp on the effective world-space angular acceleration that is passed to the simulation.
	// Units are radian/s/s. The default value effectively means "unlimited". This has a similar
	// effect to MaxAngularVelocity, except that it is related to the flying out of bodies when the
	// rotation speed suddenly changes. A typical value for a character might be around 100.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularAcceleration"))
	float MaxAngularAcceleration = 10000;

	// If the linear acceleration is above this threshold, the movement is treated as a teleport.
	// The calculated accelerations and velocities will be set to zero, without resetting the
	// simulation state.
	// A value of zero will disable this detection
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float LinearAccelerationThresholdForTeleport = 100000;

	// If the angular acceleration (deg/s/s) is above this threshold, the movement is treated as a teleport.
	// The calculated accelerations and velocities will be set to zero, without resetting the
	// simulation state.
	// A value of zero will disable this detection
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float AngularAccelerationThresholdForTeleport = 100000;

	// If the linear position change is above this threshold, the movement is treated as a teleport.
	// The calculated accelerations and velocities will be set to zero, without resetting the
	// simulation state.
	// A value of zero will disable this detection
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float PositionChangeThresholdForTeleport = 100;

	// If the angular change (degrees) is above this threshold, the movement is treated as a teleport.
	// The calculated accelerations and velocities will be set to zero, without resetting the
	// simulation state.
	// A value of zero will disable this detection
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0"))
	float OrientationChangeThresholdForTeleport = 30;

	// How much of the simulation frame's linear velocity to pass onto the bodies (linear ether
	// drag). This relies on there being drag authored on the bodies. If set to zero, the only drag
	// will be from the body's local movement (in the simulation space).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LinearDragMultiplier = 1.0;

	// How much of the simulation frame's angular velocity to pass onto the bodies (angular ether
	// drag). This relies on there being drag authored on the bodies. If set to zero, the only drag
	// will be from the body's local movement (in the simulation space).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AngularDragMultiplier = 1.0f;

	// Additional linear drag from movement of the simulation space, applied to every body in
	// addition to linear drag specified on them in the physics asset. When combined with
	// ExternalLinearVelocity, this can be used to add a temporary wind-blown effect without having
	// to tune linear drag on all the bodies in the physics asset. The result is that each body has
	// a force equal to -ExternalLinearDrag * ExternalLinearVelocity applied to it, in addition to
	// all other forces. The vector is in simulation local space.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalLinearDrag = FVector::ZeroVector;

	// Additional velocity that is added to the component velocity so the simulation acts as if the
	// actor is moving at speed, even when stationary. The vector is in world space. This could be
	// used for wind effects etc. Typical values are similar to the velocity of the object or
	// effect, and usually around or less than 1000 for characters/wind.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalLinearVelocity = FVector::ZeroVector;

	// Additional angular velocity that is added to the component angular velocity. This can be used
	// to make the simulation act as if the actor is rotating even when it is not. E.g., to apply
	// physics to a character on a podium as the camera rotates around it, to emulate the podium
	// itself rotating. Vector is in world space. Units are deg/s.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalAngularVelocity = FVector::ZeroVector;

	// This will treat the external velocity like a wind field and add turbulence to it. Units are
	// the same as velocity, so this is the approximate magnitude of the turbulence.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalTurbulenceVelocity = FVector::ZeroVector;
};

// Master gain plus per-term gains on the four pseudo-forces that Chaos Immediate Physics applies in
// non-world simulation space. The master Amount continues to act through input-vector pre-scaling
// (legacy Physics path); the per-term gains wire directly into the matching
// Chaos::FSimulationSpaceSettings::*Alpha fields. Field names mirror FRigDynamicsInertialForceSettings
// exactly. Effective gain on each pseudo-force term is Amount * <PerTerm>Amount.
USTRUCT(BlueprintType)
struct FRigPhysicsInertialForceSettings
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsInertialForceSettings& Data);

	// Master multiplier on the effects of simulation space movement on the simulation. Acts as a
	// uniform scalar applied alongside the per-term gains below.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Amount = 1.0f;

	// Gain on the linear Euler pseudo-force (response to the actor's linear acceleration; -a_frame
	// term). Wires to Chaos::FSimulationSpaceSettings::LinearAccelerationAlpha.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LinearEulerAmount = 1.0f;

	// Gain on the angular Euler pseudo-force (alpha_frame x r term). Wires to
	// Chaos::FSimulationSpaceSettings::EulerAlpha.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AngularEulerAmount = 1.0f;

	// Gain on the centrifugal pseudo-force (-omega x (omega x r) term). Wires to
	// Chaos::FSimulationSpaceSettings::CentrifugalAlpha.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float CentrifugalAmount = 1.0f;

	// Gain on the Coriolis pseudo-force (-2 omega x v_p term). Wires to
	// Chaos::FSimulationSpaceSettings::CoriolisAlpha.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float CoriolisAmount = 1.0f;
};

// Drag-related settings applied to the whole simulation due to motion of the simulation space.
// Wraps the Chaos FSimulationSpaceSettings ether-drag fields plus the world-space wind/turbulence
// vectors. ExternalLinearDrag is a Chaos-only quantity in simulation-local space and has no
// Dynamics analogue.
USTRUCT(BlueprintType)
struct FRigPhysicsSimulationDragSettings
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationDragSettings& Data);

	// How much of the simulation frame's linear velocity to pass onto the bodies (linear ether
	// drag). This relies on there being drag authored on the bodies. If set to zero, the only drag
	// will be from the body's local movement (in the simulation space).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LinearDragMultiplier = 1.0f;

	// How much of the simulation frame's angular velocity to pass onto the bodies (angular ether
	// drag). This relies on there being drag authored on the bodies. If set to zero, the only drag
	// will be from the body's local movement (in the simulation space).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AngularDragMultiplier = 1.0f;

	// Additional linear drag from movement of the simulation space, applied to every body in
	// addition to linear drag specified on them in the physics asset. When combined with
	// ExternalLinearVelocity, this can be used to add a temporary wind-blown effect without having
	// to tune linear drag on all the bodies in the physics asset. The result is that each body has
	// a force equal to -ExternalLinearDrag * ExternalLinearVelocity applied to it, in addition to
	// all other forces. The vector is in simulation local space.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalLinearDrag = FVector::ZeroVector;

	// Additional velocity that is added to the component velocity so the simulation acts as if the
	// actor is moving at speed, even when stationary. The vector is in world space. This could be
	// used for wind effects etc. Typical values are similar to the velocity of the object or
	// effect, and usually around or less than 1000 for characters/wind.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalLinearVelocity = FVector::ZeroVector;

	// Additional angular velocity that is added to the component angular velocity. This can be used
	// to make the simulation act as if the actor is rotating even when it is not. E.g., to apply
	// physics to a character on a podium as the camera rotates around it, to emulate the podium
	// itself rotating. Vector is in world space. Units are deg/s.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalAngularVelocity = FVector::ZeroVector;

	// This will treat the external velocity like a wind field and add turbulence to it. Units are
	// the same as velocity, so this is the approximate magnitude of the turbulence.
	UPROPERTY(EditAnywhere, Category = Settings)
	FVector ExternalTurbulenceVelocity = FVector::ZeroVector;
};

// Conditioning of simulation-space velocity and acceleration before they reach the inertial-force
// and drag systems. Wraps the per-axis clamps and the vertical motion scale, and contains nested
// InertialForces and Drag substructs.
USTRUCT(BlueprintType)
struct FRigPhysicsSimulationSpaceMotion
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsSimulationSpaceMotion& Data);

	// Multiplier on the vertical (Z) components of linear velocity AND acceleration of the
	// simulation space that is passed to the simulation. Usually from 0.0 to 1.0 to reduce the
	// effects of jumping and crouching on the simulation, but it can be higher than 1.0 if you
	// need to exaggerate this motion for some reason.
	UPROPERTY(EditAnywhere, Category = Settings)
	float VerticalMotionScale = 1.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearVelocity = false;

	// A clamp on the effective world-space velocity that is passed to the simulation. The default
	// value effectively means "unlimited". It is not usually required to change this but you would
	// reduce this to limit the effects of drag on the bodies in the simulation.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearVelocity"))
	float MaxLinearVelocity = 10000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularVelocity = false;

	// A clamp on the effective world-space angular velocity that is passed to the simulation. Units
	// are radian/s, so a value of about 6.0 is one rotation per second. The default value
	// effectively means "unlimited".
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularVelocity"))
	float MaxAngularVelocity = 10000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearAcceleration = false;

	// A clamp on the effective world-space acceleration that is passed to the simulation. The
	// default value effectively means "unlimited". This property is used to stop the bodies of the
	// simulation flying out when suddenly changing linear speed.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearAcceleration"))
	float MaxLinearAcceleration = 10000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularAcceleration = false;

	// A clamp on the effective world-space angular acceleration that is passed to the simulation.
	// Units are radian/s/s. The default value effectively means "unlimited".
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularAcceleration"))
	float MaxAngularAcceleration = 10000.0f;

	// Master and per-term gains on the simulation-space pseudo-forces (Linear-Euler, Angular-Euler,
	// Centrifugal, Coriolis).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FRigPhysicsInertialForceSettings InertialForces;

	// Drag and external-velocity settings (wind, turbulence, sim-local Chaos ether drag).
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ShowOnlyInnerProperties))
	FRigPhysicsSimulationDragSettings Drag;
};

// Per-channel teleport-detection thresholds applied to raw simulation-space deltas. When any
// enabled threshold is crossed, the solver zeroes velocities/accelerations for the frame without
// resetting the simulation pose.
USTRUCT(BlueprintType)
struct FRigPhysicsTeleportDetectionSettings
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsTeleportDetectionSettings& Data);

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromPositionChange = true;

	// If the linear position change (cm) is above this threshold, the movement is treated as a
	// teleport.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromPositionChange"))
	float PositionChangeThreshold = 100.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromOrientationChange = true;

	// If the angular change (degrees) is above this threshold, the movement is treated as a
	// teleport.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromOrientationChange"))
	float OrientationChangeThreshold = 30.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromLinearAcceleration = true;

	// If the linear acceleration is above this threshold, the movement is treated as a teleport.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromLinearAcceleration"))
	float LinearAccelerationThreshold = 100000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromAngularAcceleration = true;

	// If the angular acceleration (deg/s/s) is above this threshold, the movement is treated as a
	// teleport.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromAngularAcceleration"))
	float AngularAccelerationThreshold = 100000.0f;
};

// Configuration/settings for the physics solver
USTRUCT(BlueprintType)
struct FRigPhysicsSolverSettings
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsSolverSettings& Data);

	// If true, then any physics body component below (the element owning this one) will be automatically
	// added to this solver, if its UseAutomaticSolver/AutoFindSolver flag is set.
	UPROPERTY(EditAnywhere, Category = SolverSettings)
	bool bAutomaticallyAddPhysicsBodyComponents = true;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	ERigPhysicsSimulationSpace SimulationSpace = ERigPhysicsSimulationSpace::Component;

	// The space in which collision shapes are defined
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	ERigPhysicsSimulationSpace CollisionSpace = ERigPhysicsSimulationSpace::Component;

	// The bone to use for the collision or simulation space (only relevant if one is set to BoneSpace)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	FRigElementKey SpaceBone;

	// The collision shapes defined in the collision space - e.g. for representing a ground etc
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ShowOnlyInnerProperties))
	FRigPhysicsCollision Collision;

	UPROPERTY(EditAnywhere, Category = SolverSettings)
	FVector Gravity = FVector(0, 0, -981.0);

	// Overlap type used to collect world objects. If set to None then there will be no world collisions.
	UPROPERTY(EditAnywhere, Category = SolverSettings)
	ERigPhysicsWorldCollisionType WorldCollisionType = ERigPhysicsWorldCollisionType::None;

	// Should world bodies with QueryAndPhysics collision type enabled be collided against.
	// Only applies when WorldCollisionType is not none.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (EditCondition = "WorldCollisionType != ERigPhysicsWorldCollisionType::None"))
	bool bWorldCollisionIncludePhysics = true;
	
	// Should world bodies with Query collision types enabled be collided against.
	// Only applies when WorldCollisionType is not none.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (EditCondition = "WorldCollisionType != ERigPhysicsWorldCollisionType::None"))
	bool bWorldCollisionIncludeQuery = false;
	
	// Should world bodies with Probe collision types enabled be collided against.
	// Only applies when WorldCollisionType is not none.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (EditCondition = "WorldCollisionType != ERigPhysicsWorldCollisionType::None"))
	bool bWorldCollisionIncludeProbe = false;

	// Scales up the volume used to collect world objects for collision
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 1))
	float WorldCollisionBoundsExpansion = 1.1f;

	// The number of position iterations to run. The position solve is responsible for de-penetration.
	// Increasing this will improve simulation stability, but increase the cost.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	int32 PositionIterations = 8;

	// The number of velocity iterations to run. The velocity solve is responsible for restitution (bounce) and friction.
	// This should usually be a low value (even 1), but could be 0 if you don't care about friction and restitution.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	int32 VelocityIterations = 4;

	// The number of projection iterations to run. The projection phase is a final pass over the constraints, applying
	// a semi-physical correction to any joint errors remaining after the position and velocity solves. It can be
	// very helpful to stabilize joint chains, but can cause issues with collision response. The projection magnitude
	// can be controlled per-constraint in the constraint settings (assuming ProjectionIteration is not zero).
	// This should be left as 1 in almost all cases.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	int32 ProjectionIterations = 1;

	// This sets how or if the step size should be smoothed. A value of one will disable smoothing,
	// so that the physics simulation will match the Control Rig delta time. 
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 1))
	int32 MaxNumRollingAverageStepTimes = 1;

	// This is the distance margin around shapes used to detect collisions. Increasing this will increase the
	// number of inactive contacts, which will reduce the likelihood of penetration, but will also
	// increase solver cost.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float CollisionBoundsExpansion = 2.0f;

	// Expands the shape bounds to detect potentially active contacts when moving. Increasing this
	// (typically up to 1) will increase the number of inactive contacts, which will reduce the
	// likelihood of penetration, but will also increase solver cost.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float BoundsVelocityMultiplier = 1.0f;

	// The maximum margin added due to movement. Reducing this can prevent excessive numbers of inactive
	// contacts being generated.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float MaxVelocityBoundsExpansion = 25.0f;

	// When bodies are penetrating, this is the maximum velocity delta that can be applied in one frame.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float MaxDepenetrationVelocity = 0.0f;

	// The recommended fixed timestep for the solver. Set to 0 to run with variable timestep, and
	// set MaxDeltaTime to the desired sub-step time if you want to take sub-steps.
	//
	// NOTE: If this value is non-zero and less than the current frame time, the simulation will step multiple times
	// which increases the cost.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float FixedTimeStep = 0.02f;

	// The maximum number of solver steps that can be made
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 1))
	int32 MaxTimeSteps = 10;

	// If a variable timestep is in use, this is the maximum delta time that can be used. If this is smaller than the
	// requested total delta time then multiple steps will be used.
	UPROPERTY(EditAnywhere, Category = SolverSettings, meta = (ClampMin = 0))
	float MaxDeltaTime = 1.0f;

	// Whether to use the linear or non-linear solver for RBAN Joints. The linear solver is significantly cheaper than
	// the non-linear solver when you are running multiple iterations, but is more likely to suffer from jitter.
	// In general you should try to use the linear solver and increase the PositionIterations to improve stability if
	// possible, only using the non-linear solver as a last resort.
	UPROPERTY(EditAnywhere, Category = SolverSettings)
	bool bUseLinearJointSolver = true;

	// When solving joints, whether to solve the positions last (as opposed to the orientations)
	UPROPERTY(EditAnywhere, Category = SolverSettings)
	bool bSolveJointPositionsLast = true;

	// It enables the use of multi-point contact manifolds, which are created only once at the start of each tick.
	// When disabled, a single-point contact is generated in each solver iteration which is more expensive.
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bUseManifolds = true;

	// Allows the solver to use the EnableCCD setting on the bodies
	UPROPERTY(EditAnywhere, Category = CollisionSettings)
	bool bAllowCCD = false;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bResetFromPosition = false;

	// If any object in the simulation exceeds this distance from the simulation origin, we will
	// reset the pose and velocity of the entire simulation. This is to detect problems/explosions.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bResetFromPosition"))
	float PositionThresholdForReset = 10000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bResetFromKinematicSpeed = false;

	// If any kinematic object in the simulation exceeds this speed, we will reset the velocities
	// of all objects in the simulation. This is to detect problems - for example if the
	// target animation has teleports etc.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bResetFromKinematicSpeed"))
	float KinematicSpeedThresholdForReset = 5000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bResetFromKinematicAcceleration = false;

	// If any kinematic object in the simulation exceeds this acceleration, we will reset the
	// velocities of all objects in the simulation. This is to detect problems - for example if the
	// target animation has teleports etc. A value of around 100000 will typically be a good place
	// to start.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bResetFromKinematicAcceleration"))
	float KinematicAccelerationThresholdForReset = 100000.0f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bResetFromEvaluationInterval = false;

	// If the time between successive solver evaluations (measured via the rig's absolute time)
	// exceeds this value AND is also greater than the frame's delta time, the pose and velocities
	// of all objects in the simulation are reset. This detects cases where evaluation was genuinely
	// skipped - e.g. the rig stopped being ticked, or the editor was paused, without triggering
	// just because the frame's delta time was rather large.
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bResetFromEvaluationInterval", Units = "Seconds"))
	float EvaluationIntervalThresholdForReset = 0.0f;
};

// Properties of a body affecting its dynamics (movement without regard to collision).
USTRUCT(BlueprintType)
struct FRigPhysicsDynamics
{
	GENERATED_BODY()

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsDynamics& Data);

	// Material density in g/cm^3, used to derive mass from the body's collision volume. Default 1.0
	// corresponds to water. Only used when MassOverride is zero or negative.
	UPROPERTY(EditAnywhere, Category = Dynamics, meta = (EditCondition = "MassOverride <= 0"))
	float Density = 1.0f;

	// Explicit total mass for this body in kg. When positive this overrides the density-derived
	// mass; when zero or negative the mass is computed from Density and the collision volume.
	UPROPERTY(EditAnywhere, Category = Dynamics)
	float MassOverride = 1.0f;

	// When true, CentreOfMassOverride specifies the centre of mass position directly instead of
	// computing it from the collision shapes.
	UPROPERTY(EditAnywhere, Category = Dynamics)
	bool bOverrideCentreOfMass = false;

	// Centre of mass in the body's local frame. Used only when bOverrideCentreOfMass is true.
	UPROPERTY(EditAnywhere, Category = Dynamics, meta = (EditCondition = "bOverrideCentreOfMass"))
	FVector CentreOfMassOverride = FVector::ZeroVector;

	// Offset added to the calculated (or overridden) centre of mass, in the body's local frame.
	UPROPERTY(EditAnywhere, Category = Dynamics)
	FVector CentreOfMassNudge = FVector::ZeroVector;

	// When true, MomentsOfInertiaOverride specifies the diagonal of the inertia tensor directly
	// instead of computing it from the collision shapes.
	UPROPERTY(EditAnywhere, Category = Dynamics)
	bool bOverrideMomentsOfInertia = false;

	// Diagonal moments of inertia (kg*cm^2) used when bOverrideMomentsOfInertia is true.
	// Off-diagonal terms of the inertia tensor are assumed zero.
	UPROPERTY(EditAnywhere, Category = Dynamics, meta = (EditCondition = "bOverrideMomentsOfInertia"))
	FVector MomentsOfInertiaOverride = FVector(1.0f, 1.0f, 1.0f);

	// Linear velocity damping (1/s). Applied as exponential decay each frame and is mass-
	// independent by default - the same setting produces identical decay for any mass. Set
	// bScaleDampingByInverseMass to scale by inverse mass for drag-like behaviour.
	UPROPERTY(EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0"))
	float LinearDamping = 0.0f;

	// Angular velocity damping (1/s). Applied as exponential decay each frame and is mass-
	// independent by default - the same setting produces identical decay for any mass. Set
	// bScaleDampingByInverseMass to scale by inverse mass for drag-like behaviour.
	UPROPERTY(EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0"))
	float AngularDamping = 0.0f;

	// When true, LinearDamping and AngularDamping are scaled by the body's inverse mass before
	// being applied. This approximates aerodynamic drag where heavier bodies decelerate less
	// under the same author setting. Not physically accurate for angular damping (true angular
	// drag depends on inverse inertia, which is a tensor) but gives a reasonable effect.
	UPROPERTY(EditAnywhere, Category = Dynamics)
	bool bScaleDampingByInverseMass = false;
};

// This represents the motor drive associated with a physics joint, that can drive the bodies
// towards a target pose.
USTRUCT(BlueprintType)
struct FRigPhysicsDriveData
{
	GENERATED_BODY()

	friend FArchive& operator<<(FArchive& Ar, FRigPhysicsDriveData& Data);

	UPROPERTY(EditAnywhere, Category = LinearDrive)
	FLinearDriveConstraint LinearDriveConstraint;

	UPROPERTY(EditAnywhere, Category = AngularDrive)
	FAngularDriveConstraint AngularDriveConstraint;

	// If true, then targets in the linear and angular drives will be applied on top of the pose
	// from animation.
	UPROPERTY(EditAnywhere, Category = Drive)
	bool bUseSkeletalAnimation = true;

	// The amount of skeletal animation velocity to use in the targets
	UPROPERTY(EditAnywhere, Category = Drive)
	float SkeletalAnimationVelocityMultiplier = 1.0f;
};

USTRUCT(BlueprintType)
struct FRigPhysicsLinearLimit
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Limit)
	FLinearConstraint LinearConstraint;
};

USTRUCT(BlueprintType)
struct FRigPhysicsConeLimit
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Limit)
	FConeConstraint ConeConstraint;
};

USTRUCT(BlueprintType)
struct FRigPhysicsTwistLimit
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Limit)
	FTwistConstraint TwistConstraint;
};

// This represents an "articulation" - a general purpose "character joint" consisting of
//   * A linear limit (a linear constraint), which is normally used to pin two bones together
//   * An angular limit (an angular constraint), which is normally used to allow a limited range of rotational movement.
//
// The joint is defined by a parent frame, which is attached to the parent body, and a child frame which is
// attached to the child body. If the limits are all zero, then these two bodies will be held in a fixed pose.
// Normally the angular limit will be relaxed to allow rotational movement around the joint position.
//
// Most bodies in a character will have one Physics Joint which connects them to their parent in the hierarchy.
// However
//   * The root of the hierarchy will not have an enabled Physics Joint
//   * There may be times when a body has additional Physics Joints.
USTRUCT(BlueprintType)
struct FRigPhysicsJointData
{
	GENERATED_BODY()

	friend FArchive& operator<<(FArchive& Ar, FRigPhysicsJointData& Data);

	UPROPERTY(EditAnywhere, Category = Joint)
	bool bEnabled = true;

	// The auto-calculated offset of the parent frame from the parent body places it at the 
	// location of the child body relative to the parent body in the initial pose.
	UPROPERTY(EditAnywhere, Category = Joint)
	bool bAutoCalculateParentOffset = true;

	// The parent frame offset applied after any auto-calculation
	UPROPERTY(EditAnywhere, Category = Joint)
	FTransform ExtraParentOffset;

	// The auto-calculated offset of the child frame is for it to be co-located with the child body.
	UPROPERTY(EditAnywhere, Category = Joint)
	bool bAutoCalculateChildOffset = true;

	// The child frame offset applied after any auto-calculation
	UPROPERTY(EditAnywhere, Category = Joint)
	FTransform ExtraChildOffset;

	UPROPERTY(EditAnywhere, Category = Joint)
	FLinearConstraint LinearConstraint;

	UPROPERTY(EditAnywhere, Category = Joint)
	FConeConstraint ConeConstraint;

	UPROPERTY(EditAnywhere, Category = Joint)
	FTwistConstraint TwistConstraint;

	// Disable collisions between the parent and child bodies
	UPROPERTY(EditAnywhere, Category = Joint)
	bool bDisableCollision = true;

	// The amount of linear projection to reduce joint separation when the linear constraint is
	// locked or has a hard limit. A value of one will apply full projection, but can introduce artefacts.
	UPROPERTY(EditAnywhere, Category = Joint, meta = (ClampMin = 0, ClampMax = 1))
	float LinearProjectionAmount = 0.5f;

	// The amount of angular projection to reduce joint separation when the angular constraint is
	// locked or has a hard limit. Note that projecting back to angular limits will often break the
	// position projection, so it is normally better to disable this.
	UPROPERTY(EditAnywhere, Category = Joint, meta = (ClampMin = 0, ClampMax = 1))
	float AngularProjectionAmount = 0.0f;

	// As this is reduced to 0, the parent body becomes less affected by the motion of the child
	// body, making the joint behave in one direction.
	UPROPERTY(EditAnywhere, Category = Joint, meta = (ClampMin = 0))
	float ParentInverseMassScale = 1.0f;
};

// Basic settings of the body in relation to the solver that are not covered elsewhere
USTRUCT(BlueprintType)
struct FRigPhysicsBodySolverSettings
{
	GENERATED_BODY()

	FRigPhysicsBodySolverSettings(
		FRigComponentKey InPhysicsSolverComponentKey = FRigComponentKey(), FRigElementKey InTargetBone = FRigElementKey())
		: PhysicsSolverComponentKey(InPhysicsSolverComponentKey), TargetBone(InTargetBone) {}

	friend FArchive& operator <<(FArchive& Ar, FRigPhysicsBodySolverSettings& Data);

	// Note that setting the solver component, if known, has the benefit of avoiding the need to
	// search for an automatic solver.
	UPROPERTY(EditAnywhere, Category = Solver)
	FRigComponentKey PhysicsSolverComponentKey;

	// If true (and the physics solver is not explicitly set), then this component will be added to
	// any physics solver that exists above it in the hierarchy, if that solver allows automatically
	// adding physics components.
	UPROPERTY(EditAnywhere, DisplayName="Auto Find Solver", Category = Solver)
	bool bUseAutomaticSolver = true;

	// The bone that is used to initialize physics, as well as what to track when the body is set to be kinematic.
	// Note that if this is unset, the it will default to the parent of our owner.
	UPROPERTY(EditAnywhere, Category = Solver)
	FRigElementKey SourceBone;

	// The bone that is written to following simulation. Note that if this is unset, the it will
	// default to the parent of our owner.
	UPROPERTY(EditAnywhere, Category = Solver)
	FRigElementKey TargetBone;

	// Whether to include this bone in checks for whether we should reset physics on the whole rig,
	// using thresholds in the solver. Set to false for bodies you know may jump around safely when kinematic.
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bIncludeInChecksForReset = true;

	UE_API void OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey);
};

#undef UE_API
