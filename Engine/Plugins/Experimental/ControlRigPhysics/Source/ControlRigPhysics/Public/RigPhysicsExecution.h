// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsBodyComponent.h"
#include "RigPhysicsSolverComponent.h"
//#include "RigPhysicsSimulation.h"

#include "Units/Execution/RigUnit_DynamicHierarchy.h"

#include "RigPhysicsExecution.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// Base struct for all other mutable physics nodes
USTRUCT(meta = (Category = "RigPhysics", NodeColor = "1.0 0.6 0.3", Keywords = "Physics", DocumentationPolicy = "Strict"))
struct FRigUnit_PhysicsBaseMutable: public FRigUnitMutable
{
	GENERATED_BODY()
};

// Base struct for all other non-mutable physics nodes
USTRUCT(meta = (Category = "RigPhysics", NodeColor = "1.0 0.6 0.3", Keywords = "Physics", DocumentationPolicy = "Strict"))
struct FRigUnit_PhysicsBase : public FRigUnit
{
	GENERATED_BODY()
};

// Adds a set of physics components including the body, joint and controls.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Spawn Physics Components", Keywords = "Add,Construction,Create,New,Body,Joint,Control", Varying))
struct FRigUnit_AddPhysicsComponents : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddPhysicsComponents()
	{
		Owner.Type = ERigElementType::Bone;
		Solver.PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		Solver.PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// Whether to add a Physics Joint
	UPROPERTY(meta = (Input))
	bool bAddJoint = true;

	// Whether to add a simulation-space Physics Control
	UPROPERTY(meta = (Input))
	bool bAddSimSpaceControl = true;

	// Whether to add a parent-space Physics Control
	UPROPERTY(meta = (Input))
	bool bAddParentSpaceControl = true;

	// The component key of the Physics Body
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsBodyComponentKey;

	// The component key of the Physics Joint, if created
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsJointComponentKey;

	// The component key of the simulation-space Physics Control, if created
	UPROPERTY(meta = (Output))
	FRigComponentKey SimSpaceControlComponentKey;

	// The component key of the parent-space Physics Control, if created
	UPROPERTY(meta = (Output))
	FRigComponentKey ParentSpaceControlComponentKey;

	// The solver to relate this new physics element to
	UPROPERTY(meta = (Input))
	FRigPhysicsBodySolverSettings Solver;

	// The dynamics properties of the new physics body	
	UPROPERTY(meta = (Input))
	FRigPhysicsDynamics Dynamics;

	// The collision properties of the new physics body
	UPROPERTY(meta = (Input))
	FRigPhysicsCollision Collision;

	// The runtime modifiable data of the new physics body
	UPROPERTY(meta = (Input))
	FPhysicsControlModifierData BodyData;

	// The properties of the joint
	UPROPERTY(meta = (Input))
	FRigPhysicsJointData JointData;

	// Optional motor/drive associated with the physics joint
	UPROPERTY(meta = (Input))
	FRigPhysicsDriveData DriveData;

	// Data for the simulation space control
	UPROPERTY(meta = (Input))
	FPhysicsControlData SimSpaceControlData;

	// Data for the parent space control
	UPROPERTY(meta = (Input))
	FPhysicsControlData ParentSpaceControlData;
};

// Imports/creates bones from the physics asset and creates collision for them.
// The bones will lose their hierarchy and be placed under the specified parent - ready to be moved around.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Import Collision From Physics Asset", Keywords = "Construction,Create,New", Varying))
struct FRigUnit_HierarchyImportCollisionFromPhysicsAsset : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyImportCollisionFromPhysicsAsset()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Note that setting the solver component, if known, has the benefit of avoiding the need to
	// search for an automatic solver.
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// If true (and the physics solver is not explicitly set), then this component will be added to
	// any physics solver that exists above it in the hierarchy, if that solver allows automatically
	// adding physics components.
	UPROPERTY(meta = (Input))
	bool bUseAutomaticSolver = true;

	// The physics asset to import collision from
	UPROPERTY(meta = (Input))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// If this is empty, then all bones with bodies in the physics asset will be created. Otherwise
	// only bodies that relate to the specified bones will be created.
	UPROPERTY(meta = (Input, DisplayName = "Bone Mask"))
	TArray<FName> BonesToUse;

	// Prefix to the bone names
	UPROPERTY(meta = (Input))
	FName NameSpace = "Physics_";

	// Parent/owner for all the new bones
	UPROPERTY(meta = (Input))
	FRigElementKey Owner;

	// The element keys of the bones that were created to own the physics bodies
	UPROPERTY(meta = (Output))
	TArray<FRigElementKey> BoneKeys;

	// The Physics Body component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> PhysicsBodyComponentKeys;
};

// Creates multiple physics components based on the supplied physics asset.
// Note that the resulting simulation bodies may not precisely match the physics asset.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Instantiate From Physics Asset", Keywords = "Construction,Create,New", Varying))
struct FRigUnit_HierarchyInstantiateFromPhysicsAsset : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchyInstantiateFromPhysicsAsset()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Note that setting the solver component, if known, has the benefit of avoiding the need to
	// search for an automatic solver.
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// If true (and the physics solver is not explicitly set), then this component will be added to
	// any physics solver that exists above it in the hierarchy, if that solver allows automatically
	// adding physics components.
	UPROPERTY(meta = (Input))
	bool bUseAutomaticSolver = true;

	// The Physics Asset to instantiate from
	UPROPERTY(meta = (Input))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// Name of the constraint profile to use. If empty (or invalid), the default profile will be used
	UPROPERTY(meta = (Input))
	FName ConstraintProfileName;

	// If this is empty, then all bodies in the physics asset that match a bone in the hierarchy
	// will be created. Otherwise only bodies that relate to the specified bones will be created.
	UPROPERTY(meta = (Input, DisplayName = "Bone Mask"))
	TArray<FRigElementKey> BonesToUse;

	// Whether to enable the joints authored in the physics asset. Note that you can't have drives
	// without joints.
	UPROPERTY(meta = (Input))
	bool bEnableJoints = true;

	// Whether to enable the drives authored in the physics asset. Note that if you are creating
	// parent space controls, you may not want the drives
	UPROPERTY(meta = (Input))
	bool bEnableDrives = true;

	// Whether to create a simulation-space control for each body that was created
	UPROPERTY(meta = (Input))
	bool bAddSimSpaceControl = false;

	// Whether to create a parent-space control for each body that was created. Note that if this is
	// set to true, then it will be created for the root-most body in the physics asset too. This is
	// often not needed, so make sure it isn't enabled if you don't want it to be.
	UPROPERTY(meta = (Input))
	bool bAddParentSpaceControl = false;

	// If true, and BonesToUse/BoneMask is being used to create a subset of the physics asset, then helper
	// bodies will be created when needed: Kinematic bodies on bones that are connected by a physics
	// joint to those in BonesToUse/BoneMask. This is used so that (a) instantiating individual chains can be
	// done without explicitly including the parent (in the physics joint sense) body, and (b)
	// entire physics assets can be made from multiple calls to instantiate individual chains.
	UPROPERTY(meta = (Input))
	bool bMakeHelperBodies = true;

	// Data for the simulation space control
	UPROPERTY(meta = (Input))
	FPhysicsControlData SimSpaceControlData;

	// Data for the parent space control
	UPROPERTY(meta = (Input))
	FPhysicsControlData ParentSpaceControlData;

	// The Physics Body component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> PhysicsBodyComponentKeys;

	// The Physics Joint component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> PhysicsJointComponentKeys;

	// The simulation-space Physics Control component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> SimSpaceControlComponentKeys;

	// The parent-space Physics Control component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> ParentSpaceControlComponentKeys;
};

#undef UE_API
