// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsData.h"
#include "RigPhysicsBodyComponent.h"

#include "RigPhysicsJointComponent.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

enum class ERigPhysicsJointComponentDirtyFlags : uint8
{
	None  = 0,
	Joint = 1 << 0,
	Drive = 1 << 1,
	All   = Joint | Drive
};
ENUM_CLASS_FLAGS(ERigPhysicsJointComponentDirtyFlags);

USTRUCT(BlueprintType)
struct FRigPhysicsJointComponent : public FRigBaseComponent
{
public:

	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigPhysicsJointComponent)

	FRigPhysicsJointComponent()
	{
		ParentBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ParentBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		ChildBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ChildBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	// The parent body of the joint. If unset, then the system will try to find a suitable body by
	// looking for a parent/grandparent of the owner etc that contains a body that is in the same
	// solver as the child body.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	FRigComponentKey ParentBodyComponentKey;

	// The child body of the joint. If unset, then the system will try to find a suitable body
	// amongst the body components under owner.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	FRigComponentKey ChildBodyComponentKey;

	// The properties of the joint
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsJointData JointData;

	// Optional motor/drive associated with the physics joint
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsDriveData DriveData;

	// Flags indicating what has been changed and needs to be set in the physics representation
	mutable ERigPhysicsJointComponentDirtyFlags DirtyFlags = ERigPhysicsJointComponentDirtyFlags::All;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	UE_API virtual bool CanBeAddedTo(
		const FRigElementKey& InElementKey, const URigHierarchy* InHierarchy, FString* OutFailureReason) const override;

	UE_API virtual void OnRigHierarchyKeyChanged(const FRigHierarchyKey& InOldKey, const FRigHierarchyKey& InNewKey) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }

	static FName GetDefaultName() { return TEXT("PhysicsJoint"); }
};

#undef UE_API
