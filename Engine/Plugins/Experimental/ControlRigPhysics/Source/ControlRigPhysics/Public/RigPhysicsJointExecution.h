// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsExecution.h"
#include "RigPhysicsJointComponent.h"

#include "RigPhysicsJointExecution.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// Adds a new Physics Joint as a component on the owner element.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta = (DisplayName = "Spawn Physics Joint", Keywords = "Add,Construction,Create,New,Joint,Articulation,Constraint,Skeleton", Varying))
struct FRigUnit_AddPhysicsJoint : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddPhysicsJoint()
	{
		Owner.Type = ERigElementType::Bone;
		ParentBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ParentBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
		ChildBodyComponentKey.ElementKey.Type = ERigElementType::Bone;
		ChildBodyComponentKey.Name = FRigPhysicsBodyComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// The Physics Joint component key that was created
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsJointComponentKey;

	// The parent body of the joint. If unset, then the system will try to find a suitable body by
	// looking for a parent/grandparent of the owner etc that contains a body that is in the same
	// solver as the child body.
	UPROPERTY(meta = (Input))
	FRigComponentKey ParentBodyComponentKey;

	// The child body of the joint. If unset, then the system will try to find a suitable body
	// amongst the body components under owner.
	UPROPERTY(meta = (Input))
	FRigComponentKey ChildBodyComponentKey;

	// The properties of the joint
	UPROPERTY(meta = (Input))
	FRigPhysicsJointData JointData;

	// Optional motor/drive associated with the physics joint
	UPROPERTY(meta = (Input))
	FRigPhysicsDriveData DriveData;
};

// Indicates whether the component exists and is a Physics Joint
USTRUCT(meta=(DisplayName="Get Physics Joint Exists", Keywords="Query,Joint,Articulation,Constraint"))
struct FRigUnit_GetPhysicsJointExists : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigPhysicsJointComponent::GetDefaultName());

	// Whether the component exists and is a Physics Joint
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

// Sets the joint data for a physics joint component
USTRUCT(meta = (DisplayName = "Set Physics Joint Properties", Varying))
struct FRigUnit_HierarchySetJointData : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointData()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The physics joint data (limit and drive properties) to be used
	UPROPERTY(meta = (Input))
	FRigPhysicsJointData JointData;
};

// Gets the joint data for a physics joint component
USTRUCT(meta = (DisplayName = "Get Physics Joint Properties"))
struct FRigUnit_HierarchyGetJointData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetJointData()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint from which to retrieve data
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The current Physics Joint data (state of limits and drives)
	UPROPERTY(meta = (Output))
	FRigPhysicsJointData JointData;
};

// Sets the joint linear limit
USTRUCT(meta = (DisplayName = "Set Physics Joint Linear Constraint Limit", Varying))
struct FRigUnit_HierarchySetJointLinearLimit : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointLinearLimit()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The linear constraint to be used
	UPROPERTY(meta = (Input))
	FRigPhysicsLinearLimit LinearLimit;
};

// Sets the joint cone limit
USTRUCT(meta = (DisplayName = "Set Physics Joint Cone Constraint Limit", Varying))
struct FRigUnit_HierarchySetJointConeLimit : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointConeLimit()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The cone limit
	UPROPERTY(meta = (Input))
	FRigPhysicsConeLimit ConeLimit;
};

// Sets the joint twist limit
USTRUCT(meta = (DisplayName = "Set Physics Joint Twist Constraint Limit", Varying))
struct FRigUnit_HierarchySetJointTwistLimit : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointTwistLimit()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The twist limit
	UPROPERTY(meta = (Input))
	FRigPhysicsTwistLimit TwistLimit;
};

// Specifies whether a Physics Joint should be enabled or not
USTRUCT(meta = (DisplayName = "Set Physics Joint Enabled", Varying))
struct FRigUnit_HierarchySetJointEnabled : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointEnabled()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// Whether or not the joint should be enabled. Disabling the joint will disable any limits and drives
	UPROPERTY(meta = (Input))
	bool bEnabled = true;
};

// Sets the joint drive for a physics component body
USTRUCT(meta = (DisplayName = "Set Physics Joint Drive Properties", Varying))
struct FRigUnit_HierarchySetJointDriveData : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointDriveData()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The Physics Joint drive data that should be used
	UPROPERTY(meta = (Input))
	FRigPhysicsDriveData DriveData;
};

// Gets the joint drive for a physics joint component
USTRUCT(meta = (DisplayName = "Get Physics Joint Drive Properties"))
struct FRigUnit_HierarchyGetJointDriveData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_HierarchyGetJointDriveData()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint from which to retrieve data
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// The current Physics Joint Drive data
	UPROPERTY(meta = (Output))
	FRigPhysicsDriveData DriveData;
};

// Sets the joint drive for a physics component body
USTRUCT(meta = (DisplayName = "Set Physics Joint Drive Use Skeletal Animation", Varying))
struct FRigUnit_HierarchySetJointDriveUseSkeletalAnimation : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetJointDriveUseSkeletalAnimation()
	{
		PhysicsJointComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsJointComponentKey.Name = FRigPhysicsJointComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Joint to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsJointComponentKey;

	// Whether the drive targets should be relative to the incoming animation pose. If set to true
	// (and the drive targets are zero), then the drives will track the target animation.
	UPROPERTY(meta = (Input))
	bool bUseSkeletalAnimation = true;
};

// Helper to simplify creation of joint data for a typical articulation (i.e. locked linear motion,
// but with an angular limit)
USTRUCT(meta = (DisplayName = "Make Articulation Joint Data", Keywords = "Construction,Create,New,Body,Skeleton", Varying))
struct FRigUnit_MakeArticulationJointData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The angular limit in Degrees (twist, swing1, swing2)
	// -ve indicates the limit range is free
	UPROPERTY(meta = (Input, ClampMin = "-1.0"))
	FVector AngularLimit = FVector(-1, -1, -1);

	// If limited, then this will be used to control the softness of the limit
	// -ve indicates the limit is hard
	// A value of 1 is reasonably soft
	UPROPERTY(meta = (Input, ClampMin = "-1.0"))
	FVector SoftStrength = FVector(-1, -1, -1);

	// If limited, then this will be used to control the damping ratio of the limit
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	FVector SoftDampingRatio = FVector(1, 1, 1);

	// Joint data that can be used in a Physics Joint, calculated according to the values set here,
	// and configured to act as a "normal" articulation joint.
	UPROPERTY(meta = (Output))
	FRigPhysicsJointData JointData;
};

// Helper to simplify creation of drive data for a typical articulation (i.e. no linear drive, but
// with an angular drive)
USTRUCT(meta = (DisplayName = "Make Articulation Drive Data", Keywords = "Construction,Create,New,Body,Skeleton", Varying))
struct FRigUnit_MakeArticulationDriveData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	UPROPERTY(meta = (Input))
	// Whether to enable the angular drive
	bool bEnableAngularDrive = true;

	// The type of drive. Note that SLERP drives don't work if any axis is locked
	UPROPERTY(meta = (Input))
	TEnumAsByte<enum EAngularDriveMode::Type> AngularDriveMode = EAngularDriveMode::SLERP;

	// The strength used to drive angular motion
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularStrength = 10.0f;

	// The amount of damping associated with the angular strength. A value of 1 Results in
	// critically damped motion where the control drives as quickly as possible to the target
	// without overshooting. Values > 1 result in more damped motion, and values below 1 result in
	// faster, but more "wobbly" motion.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularDampingRatio = 1.0f;

	// The amount of additional angular damping. This is added to the damping that comes from
	// AngularDampingRatio and can be useful when you want damping even when AngularStrength is zero.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularExtraDamping = 0.0f;

	// The amount of skeletal animation velocity to use in the targets
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float SkeletalAnimationVelocityMultiplier = 1.0f;

	// The drive data that can be set in a Physics Joint, reflecting the setup specified here.
	UPROPERTY(meta = (Output))
	FRigPhysicsDriveData DriveData;
};

// Helper to simplify creation of drive data
USTRUCT(meta = (DisplayName = "Make Drive Data", Keywords = "Construction,Create,New,Body,Skeleton", Varying))
struct FRigUnit_MakeDriveData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Whether to enable the linear drive (not normally used for character joints)
	UPROPERTY(meta = (Input))
	bool bEnableLinearDrive = false;

	// The strength used to drive linear motion
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LinearStrength = 10.0f;

	// The amount of damping associated with the linear strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without
	// overshooting. Values > 1 result in more damped motion, and values below 1 result in faster,
	// but more "wobbly" motion.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LinearDampingRatio = 1.0f;

	// The amount of additional linear damping. This is added to the damping that comes from
	// LinearDampingRatio and can be useful when you want damping even when LinearStrength is zero.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LinearExtraDamping = 0.0f;

	// Whether to enable the angular drive
	UPROPERTY(meta = (Input))
	bool bEnableAngularDrive = true;

	// The type of drive. Note that SLERP drives don't work if any axis is locked
	UPROPERTY(meta = (Input))
	TEnumAsByte<enum EAngularDriveMode::Type> AngularDriveMode = EAngularDriveMode::SLERP;

	// The strength used to drive angular motion 
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularStrength = 10.0f;

	// The amount of damping associated with the angular strength. A value of 1 Results in critically
	// damped motion where the control drives as quickly as possible to the target without overshooting.
	// Values > 1 result in more damped motion, and values below 1 result in faster, but more "wobbly" motion.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularDampingRatio = 1.0f;

	// The amount of additional angular damping. This is added to the damping that comes from AngularDampingRatio
	// and can be useful when you want damping even when AngularStrength is zero.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float AngularExtraDamping = 0.0f;

	// The amount of skeletal animation velocity to use in the targets
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float SkeletalAnimationVelocityMultiplier = 1.0f;

	// The drive data, representing the properties specified here, that can be set on a Physics Joint
	UPROPERTY(meta = (Output))
	FRigPhysicsDriveData DriveData;
};

#undef UE_API
