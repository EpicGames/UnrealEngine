// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsData.h"

#include "Rigs/RigHierarchyComponents.h"

#include "RigPhysicsSolverComponent.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

struct FRigPhysicsSolver;

//======================================================================================================================
// A solver coordinates the physical movement of bodies
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigPhysicsSolverComponent : public FRigBaseComponent
{
public:

	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigPhysicsSolverComponent)

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsSolverSettings SolverSettings;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsSimulationSpaceMotion SpaceMotion;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Physics, meta = (ShowOnlyInnerProperties))
	FRigPhysicsTeleportDetectionSettings TeleportDetection;

	// If we have to reset, then we can be told to track the input kinematically for a number of
	// frames - this will be set > 0 and will be decremented each update. When it reaches 0, things
	// can go to the desired simulation state.
	int32 TrackInputCounter = 0;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

#if WITH_EDITOR
	UE_API virtual const FSlateIcon& GetIconForUI() const override;
#endif

	UE_API virtual void OnAddedToHierarchy(URigHierarchy* InHierarchy, URigHierarchyController* InController) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("PhysicsSolver"); }

	// This will make the internally owned solver if necessary, and return it. Note that there's no
	// danger of this being called concurrently from different threads, since each FRigPhysicsSolver
	// is owned by just one rig.
	FRigPhysicsSolver* GetPhysicsSolver() const;

private:
	// We use a TSharedPtr here to share access/ownership with the game-thread world collisions task.
	//
	// We make it mutable so that GetPhysicsSolver can be const - in practice the user shouldn't
	// have to worry.
	//
	// Finally, UE's reflection system wants to be able to use the assignment operator. If it wasn't
	// for that, we could just delete the copy operator etc and prevent duplicates containing a
	// pointer to the same PhysicsSolver. In practice, this won't ever happen for a "live"
	// component, so we're OK.
	mutable TSharedPtr<FRigPhysicsSolver> PhysicsSolverPtr;
};

#undef UE_API
