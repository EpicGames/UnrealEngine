// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigPhysicsExecution.h"

#include "RigPhysicsSolverExecution.generated.h"

#define UE_API CONTROLRIGPHYSICS_API

// Adds a new physics solver as a component on the owner element.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
//
// Deprecated in favour of FRigUnit_SpawnPhysicsSolver, which exposes the new SpaceMotion +
// TeleportDetection input pins separately. Existing graphs continue to work - the deprecated
// Execute body routes the legacy SimulationSpaceSettings into the new component members via
// ConvertLegacyPhysicsSimulationSpaceSettings; right-clicking the deprecated node offers a
// one-click upgrade to the new node via GetUpgradeInfo.
USTRUCT(meta=(DisplayName="Spawn Physics Solver", Keywords="Add,Construction,Create,New,Simulation", Varying, Deprecated="5.8"))
struct FRigUnit_AddPhysicsSolver : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_AddPhysicsSolver()
	{
		Owner.Type = ERigElementType::Bone;
		// Default the material here to have friction and restitution. Then the interactions are
		// easily adjusted on the dynamic bodies, because we prefer to combine by multiplying. If
		// restitution was zero here, then setting restitution on the dynamic bodies would have no effect.
		SolverSettings.Collision.Material.Friction = 1.0f;
		SolverSettings.Collision.Material.Restitution = 1.0f;
		SolverSettings.Collision.Material.FrictionCombineMode = ERigPhysicsCombineMode::Multiply;
		SolverSettings.Collision.Material.RestitutionCombineMode = ERigPhysicsCombineMode::Multiply;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// The component key of the solver that has been added
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsSolverComponentKey;

	// Settings for the physics solver that will be added
	UPROPERTY(meta = (Input))
	FRigPhysicsSolverSettings SolverSettings;

	// Settings for the solver that apply to when it uses a simulation space other than "world".
	// These typically relate to the movement of the simulation space itself, and how that is used
	// by the solver.
	UPROPERTY(meta = (Input))
	FRigPhysicsSimulationSpaceSettings SimulationSpaceSettings;

	// Maps the legacy SimulationSpaceSettings pin onto the FRigUnit_SpawnPhysicsSolver layout
	// (SpaceMotion + TeleportDetection sub-pins). Lets users right-click and upgrade to the new
	// node manually; the deprecated Execute body stays functional for graphs that haven't been
	// upgraded yet.
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

// Adds a new physics solver as a component on the owner element.
// Note: This node only runs during the construction event (before physics instantiation,
// after which this data is frozen).
USTRUCT(meta=(DisplayName="Spawn Physics Solver", Keywords="Add,Construction,Create,New,Simulation", Varying))
struct FRigUnit_SpawnPhysicsSolver : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SpawnPhysicsSolver()
	{
		Owner.Type = ERigElementType::Bone;
		// Default the material here to have friction and restitution. Then the interactions are
		// easily adjusted on the dynamic bodies, because we prefer to combine by multiplying. If
		// restitution was zero here, then setting restitution on the dynamic bodies would have no effect.
		SolverSettings.Collision.Material.Friction = 1.0f;
		SolverSettings.Collision.Material.Restitution = 1.0f;
		SolverSettings.Collision.Material.FrictionCombineMode = ERigPhysicsCombineMode::Multiply;
		SolverSettings.Collision.Material.RestitutionCombineMode = ERigPhysicsCombineMode::Multiply;
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner;

	// The component key of the solver that has been added
	UPROPERTY(meta = (Output))
	FRigComponentKey PhysicsSolverComponentKey;

	// Settings for the physics solver that will be added
	UPROPERTY(meta = (Input))
	FRigPhysicsSolverSettings SolverSettings;

	// Per-channel teleport-detection thresholds applied to raw simulation-space deltas.
	UPROPERTY(meta = (Input))
	FRigPhysicsTeleportDetectionSettings TeleportDetection;

	// Conditioning of simulation-space velocity/acceleration plus nested inertial-force gain and drag.
	UPROPERTY(meta = (Input))
	FRigPhysicsSimulationSpaceMotion SpaceMotion;
};

// Indicates whether the component exists and is a Physics Solver
USTRUCT(meta=(DisplayName="Get Physics Solver Exists", Keywords="Query,Simulation"))
struct FRigUnit_GetPhysicsSolverExists : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigPhysicsSolverComponent::GetDefaultName());

	// Whether the component exists and is a Physics Solver
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

// Instantiates all the objects in the physics world. Some properties can't be modified after this happens.
// Note that it will happen automatically during the first simulation step if it hasn't been explicitly
// requested. Explicit instantiation allows the timing to be controlled, as allocations etc may cause some
// delays.
USTRUCT(meta = (DisplayName = "Instantiate physics"))
struct FRigUnit_InstantiatePhysics : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_InstantiatePhysics()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The solver that should be instantiated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;
};

// Steps the specified physics solver
USTRUCT(meta = (DisplayName = "Step Physics Solver", Keywords = "Simulate", Deprecated="5.8"))
struct FRigUnit_StepPhysicsSolver : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_StepPhysicsSolver()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver that should be stepped
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;
	
	// If this is zero, then the execute context time will be used. If this is positive then it will
	// override the delta time. A negative value will prevent the solver from stepping, but there will
	// still be update costs associated with the node.
	UPROPERTY(meta = (Input))
	float DeltaTimeOverride = 0.0f;

	// If this is zero, then the simulation delta time will be used for evaluating movement of the
	// simulation space. If this is positive then it will override. This may be needed if the
	// component movement is being done in parallel, in which case you might need to pass in the
	// previous time delta here.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float SimulationSpaceDeltaTimeOverride = 0.0f;

	// How much of the simulation is combined with the input bone. This currently happens in
	// component space. Note that the simulation will continue to run, even if alpha = 0
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0f;

	// Settings that specify how the solver state should be visualized during/after the step
	UPROPERTY(meta = (Input))
	FRigPhysicsVisualizationSettings VisualizationSettings;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

// Steps the specified physics solver
USTRUCT(meta = (DisplayName = "Step Physics Solver", Keywords = "Simulate"))
struct FRigUnit_StepPhysicsSolver1 : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_StepPhysicsSolver1()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver that should be stepped
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// If this is zero, then the execute context time will be used. If this is positive then it will
	// override the delta time. A negative value will prevent the solver from stepping, but there will
	// still be update costs associated with the node.
	UPROPERTY(meta = (Input))
	float DeltaTimeOverride = 0.0f;

	// If this is zero, then the simulation delta time will be used for evaluating movement of the
	// simulation space. If this is positive then it will override. This may be needed if the
	// component movement is being done in parallel, in which case you might need to pass in the
	// previous time delta here.
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float SimulationSpaceDeltaTimeOverride = 0.0f;

	// How much of the simulation is combined with the input bone. This currently happens in
	// component space. Each body additionally has its own PhysicsBlendWeight, and the effective
	// blend per body is Alpha * PhysicsBlendWeight.
	//
	// When Alpha is zero the simulation is bypassed entirely (pass-through) - see
	// bTrackVelocitiesDuringPassThrough for the resume behaviour. Note that only Alpha triggers
	// the fast pass-through: setting every body's PhysicsBlendWeight to zero leaves the bones at
	// the input animation pose (no hierarchy writeback), but the simulation still runs. Set
	// Alpha to zero if you want to skip the simulation cost entirely.
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0f;

	// When true and Alpha is zero, the solver still tracks source bone transforms each frame so
	// that velocities are valid when Alpha increases (smooth resume), at a small per-body cost.
	// When false and Alpha is zero, the pass-through is near-zero cost but the simulation does a
	// brief warm-up reset (forcing kinematic tracking for a few frames) when Alpha increases.
	UPROPERTY(meta = (Input))
	bool bTrackVelocitiesDuringPassThrough = true;

	// Settings that specify how the solver state should be visualized during/after the step
	UPROPERTY(meta = (Input))
	FRigPhysicsVisualizationSettings1 VisualizationSettings;
};

// Sets the solver's simulation space settings.
// Note: Solver settings are pushed to the simulation each step, so this can be modified after physics instantiation.
//
// Deprecated in favour of FRigUnit_SetPhysicsSolverSpaceMotion +
// FRigUnit_SetPhysicsSolverTeleportDetection. Existing graphs continue to work - the deprecated
// Execute body routes the legacy struct into the new component members via
// ConvertLegacyPhysicsSimulationSpaceSettings.
USTRUCT(meta = (DisplayName = "Set Physics Solver Simulation Space Settings", Deprecated="5.8"))
struct FRigUnit_SetPhysicsSolverSimulationSpaceSettings : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetPhysicsSolverSimulationSpaceSettings()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The new simulation space settings
	UPROPERTY(meta = (Input))
	FRigPhysicsSimulationSpaceSettings SimulationSpaceSettings;

	// Returns an invalid upgrade info so existing graphs keep this node in place rather than
	// auto-swapping it. The Execute body remains functional.
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

// Gets the solver's simulation space settings.
// Deprecated in favour of FRigUnit_GetPhysicsSolverSpaceMotion +
// FRigUnit_GetPhysicsSolverTeleportDetection. Existing graphs continue to work - the deprecated
// Execute body builds a legacy view via BuildLegacyPhysicsSimulationSpaceSettingsView.
USTRUCT(meta = (DisplayName = "Get Physics Solver Simulation Space Settings", Deprecated="5.8"))
struct FRigUnit_GetPhysicsSolverSimulationSpaceSettings : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_GetPhysicsSolverSimulationSpaceSettings()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The current simulation space settings
	UPROPERTY(meta = (Output))
	FRigPhysicsSimulationSpaceSettings SimulationSpaceSettings;

	// Returns an invalid upgrade info so existing graphs keep this node in place rather than
	// auto-swapping it. The Execute body remains functional.
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

// Sets the solver's simulation space motion settings (conditioning + nested inertial-force / drag).
// Note: Solver settings are pushed to the simulation each step, so this can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Solver Space Motion"))
struct FRigUnit_SetPhysicsSolverSpaceMotion : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetPhysicsSolverSpaceMotion()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The new simulation space motion settings
	UPROPERTY(meta = (Input))
	FRigPhysicsSimulationSpaceMotion SpaceMotion;
};

// Gets the solver's simulation space motion settings.
USTRUCT(meta = (DisplayName = "Get Physics Solver Space Motion"))
struct FRigUnit_GetPhysicsSolverSpaceMotion : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_GetPhysicsSolverSpaceMotion()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The current simulation space motion settings
	UPROPERTY(meta = (Output))
	FRigPhysicsSimulationSpaceMotion SpaceMotion;
};

// Sets the solver's teleport-detection thresholds.
// Note: Solver settings are pushed to the simulation each step, so this can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Solver Teleport Detection"))
struct FRigUnit_SetPhysicsSolverTeleportDetection : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetPhysicsSolverTeleportDetection()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The new teleport-detection thresholds
	UPROPERTY(meta = (Input))
	FRigPhysicsTeleportDetectionSettings TeleportDetection;
};

// Gets the solver's teleport-detection thresholds.
USTRUCT(meta = (DisplayName = "Get Physics Solver Teleport Detection"))
struct FRigUnit_GetPhysicsSolverTeleportDetection : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_GetPhysicsSolverTeleportDetection()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The current teleport-detection thresholds
	UPROPERTY(meta = (Output))
	FRigPhysicsTeleportDetectionSettings TeleportDetection;
};

// Sets the external velocity of the simulation - used for adding wind effects.
// Note: Solver settings are pushed to the simulation each step, so this can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Solver External Velocity"))
struct FRigUnit_SetPhysicsSolverExternalVelocity : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetPhysicsSolverExternalVelocity()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// Additional velocity that is added to the component velocity so the simulation acts as if the
	// actor is moving at speed, even when stationary. The vector is in world space. This could be
	// used for wind effects etc. Typical values are similar to the velocity of the object or
	// effect, and usually around or less than 1000 for characters/wind.
	UPROPERTY(meta = (Input))
	FVector ExternalLinearVelocity = FVector::ZeroVector;

	// Additional angular velocity that is added to the component angular velocity. This can be used
	// to make the simulation act as if the actor is rotating even when it is not. E.g., to apply
	// physics to a character on a podium as the camera rotates around it, to emulate the podium
	// itself rotating. Vector is in world space. Units are deg/s.
	UPROPERTY(meta = (Input))
	FVector ExternalAngularVelocity = FVector::ZeroVector;

	// This will treat the external velocity like a wind field and add turbulence to it. Units are
	// the same as velocity, so this is the approximate magnitude of the turbulence.
	UPROPERTY(meta = (Input))
	FVector ExternalTurbulenceVelocity = FVector::ZeroVector;
};

// Forces tracking of the input animation (on all physics bodies) for the next N frames.
//
// Deprecated in favour of Step Physics Solver with Alpha = 0. Setting Alpha to 0 produces a
// cheaper pass-through and (when bTrackVelocitiesDuringPassThrough is false) automatically
// schedules the same warm-up tracking on resume that this node sets manually. Existing graphs
// continue to work - the Execute body still bumps TrackInputCounter as before.
USTRUCT(meta = (DisplayName = "Set Solver Track Input Pose", Keywords = "Reset,Simulate", Deprecated="5.8"))
struct FRigUnit_TrackInputPose : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_TrackInputPose()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The solver to relate this new physics element to
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The number of frames to track the input pose for
	UPROPERTY(meta = (Input, ClampMin = "0"))
	int NumberOfFrames = 1;

	// If true, then the number will be forced, potentially reducing the number. If false, then the
	// NumberOfFrames will only be used to increase the number of frames remaining.
	UPROPERTY(meta = (Input))
	bool bForceNumberOfFrames = false;

	// Returns an invalid upgrade info so existing graphs keep this node in place rather than
	// auto-swapping it. The Execute body remains functional.
	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;
};

// Retrieves the simulation space data that were generated during the simulation step, so the values
// returned will relate to the previous update if the solver has not yet been stepped.
USTRUCT(meta = (DisplayName = "Get Physics Solver Space Data", Keywords = "Debug"))
struct FRigUnit_GetPhysicsSolverSpaceData : public FRigUnit_PhysicsBase
{
	GENERATED_BODY()

	FRigUnit_GetPhysicsSolverSpaceData()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The solver to relate this new physics element to
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// The velocity of the simulation space (in world space)
	UPROPERTY(meta = (Output))
	FVector LinearVelocity = FVector::ZeroVector;

	// The angular velocity of the simulation space (in world space)
	UPROPERTY(meta = (Output))
	FVector AngularVelocity = FVector::ZeroVector;

	// The linear acceleration of the simulation space (in world space)
	UPROPERTY(meta = (Output))
	FVector LinearAcceleration = FVector::ZeroVector;

	// The angular acceleration of the simulation space (in world space)
	UPROPERTY(meta = (Output))
	FVector AngularAcceleration = FVector::ZeroVector;

	// The gravitational acceleration that will be applied (in simulation space)
	UPROPERTY(meta = (Output))
	FVector Gravity = FVector::ZeroVector;
};

// Enables/disables support for CCD in the solver.
// Note: Solver settings are pushed to the simulation each step, so this can be modified after physics instantiation.
USTRUCT(meta = (DisplayName = "Set Physics Solver Allow CCD", Varying))
struct FRigUnit_HierarchySetPhysicsSolverAllowCCD : public FRigUnit_PhysicsBaseMutable
{
	GENERATED_BODY()

	FRigUnit_HierarchySetPhysicsSolverAllowCCD()
	{
		PhysicsSolverComponentKey.ElementKey.Type = ERigElementType::Bone;
		PhysicsSolverComponentKey.Name = FRigPhysicsSolverComponent::GetDefaultName();
	}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Physics Solver to be updated
	UPROPERTY(meta = (Input))
	FRigComponentKey PhysicsSolverComponentKey;

	// Whether or not the solver allows CCD to be used. CCD also needs to be enabled on individual bodies
	UPROPERTY(meta = (Input))
	bool bAllowCCD = false;
};

#undef UE_API
