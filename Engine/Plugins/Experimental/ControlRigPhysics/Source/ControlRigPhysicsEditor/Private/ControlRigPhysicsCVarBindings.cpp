// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigPhysicsCVarBindings.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ControlRigPhysicsCVarBindings"

namespace ControlRigPhysicsEditor
{

//======================================================================================================================
// FCVarSimpleToggleBinding
//======================================================================================================================

//======================================================================================================================
FCVarSimpleToggleBinding::FCVarSimpleToggleBinding(const FString& InCVarName, const FText& InTooltip)
	: Tooltip(InTooltip)
{
	CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);
}

//======================================================================================================================
FCVarSimpleToggleBinding::~FCVarSimpleToggleBinding()
{
	if (CachedCVar && OnChangedHandle.IsValid())
	{
		CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
	}
}

//======================================================================================================================
void FCVarSimpleToggleBinding::Initialize(TWeakPtr<SWidget> InOwnerWidget)
{
	OwnerWidget = InOwnerWidget;
	if (CachedCVar && !OnChangedHandle.IsValid())
	{
		OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
			AsShared(), &FCVarSimpleToggleBinding::HandleCVarChanged);
	}
}

//======================================================================================================================
TSharedRef<SWidget> FCVarSimpleToggleBinding::BuildOverrideCell()
{
	// Simple toggles have no override - return an empty spacer so the grid column stays aligned.
	return SNew(SSpacer);
}

//======================================================================================================================
TSharedRef<SWidget> FCVarSimpleToggleBinding::BuildValueCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarSimpleToggleBinding::GetValueCheckState)
		.OnCheckStateChanged(this, &FCVarSimpleToggleBinding::OnValueCheckStateChanged)
		.IsEnabled(CachedCVar != nullptr)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
void FCVarSimpleToggleBinding::HandleCVarChanged(IConsoleVariable* /*InVariable*/)
{
	if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
	{
		Owner->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

//======================================================================================================================
ECheckBoxState FCVarSimpleToggleBinding::GetValueCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	return CachedCVar->GetInt() != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarSimpleToggleBinding::OnValueCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	const int32 NewValue = (NewState == ECheckBoxState::Checked) ? 1 : 0;
	CachedCVar->Set(NewValue, ECVF_SetByConsole);
}


//======================================================================================================================
// FCVarOverrideToggleBinding
//======================================================================================================================

//======================================================================================================================
FCVarOverrideToggleBinding::FCVarOverrideToggleBinding(const FString& InCVarName, const FText& InTooltip)
	: Tooltip(InTooltip)
{
	CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);
	if (CachedCVar)
	{
		const int32 CurrentValue = CachedCVar->GetInt();
		if (CurrentValue == 0 || CurrentValue == 1)
		{
			PendingValue = CurrentValue;
		}
	}
}

//======================================================================================================================
FCVarOverrideToggleBinding::~FCVarOverrideToggleBinding()
{
	if (CachedCVar && OnChangedHandle.IsValid())
	{
		CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
	}
}

//======================================================================================================================
void FCVarOverrideToggleBinding::Initialize(TWeakPtr<SWidget> InOwnerWidget)
{
	OwnerWidget = InOwnerWidget;
	if (CachedCVar && !OnChangedHandle.IsValid())
	{
		OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
			AsShared(), &FCVarOverrideToggleBinding::HandleCVarChanged);
	}
}

//======================================================================================================================
TSharedRef<SWidget> FCVarOverrideToggleBinding::BuildOverrideCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarOverrideToggleBinding::GetApplyCheckState)
		.OnCheckStateChanged(this, &FCVarOverrideToggleBinding::OnApplyCheckStateChanged)
		.IsEnabled(CachedCVar != nullptr)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
TSharedRef<SWidget> FCVarOverrideToggleBinding::BuildValueCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarOverrideToggleBinding::GetValueCheckState)
		.OnCheckStateChanged(this, &FCVarOverrideToggleBinding::OnValueCheckStateChanged)
		.IsEnabled(this, &FCVarOverrideToggleBinding::IsValueToggleEnabled)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
void FCVarOverrideToggleBinding::HandleCVarChanged(IConsoleVariable* /*InVariable*/)
{
	if (CachedCVar)
	{
		const int32 CurrentValue = CachedCVar->GetInt();
		if (CurrentValue == 0 || CurrentValue == 1)
		{
			PendingValue = CurrentValue;
		}
	}
	if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
	{
		Owner->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

//======================================================================================================================
ECheckBoxState FCVarOverrideToggleBinding::GetApplyCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	return CachedCVar->GetInt() >= 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarOverrideToggleBinding::OnApplyCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	if (NewState == ECheckBoxState::Checked)
	{
		CachedCVar->Set(PendingValue, ECVF_SetByConsole);
	}
	else
	{
		CachedCVar->Set(-1, ECVF_SetByConsole);
	}
}

//======================================================================================================================
ECheckBoxState FCVarOverrideToggleBinding::GetValueCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	const int32 CurrentValue = CachedCVar->GetInt();
	const int32 ValueToShow = (CurrentValue >= 0) ? CurrentValue : PendingValue;
	return ValueToShow != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarOverrideToggleBinding::OnValueCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	PendingValue = (NewState == ECheckBoxState::Checked) ? 1 : 0;
	if (CachedCVar->GetInt() >= 0)
	{
		CachedCVar->Set(PendingValue, ECVF_SetByConsole);
	}
}

//======================================================================================================================
bool FCVarOverrideToggleBinding::IsValueToggleEnabled() const
{
	return CachedCVar != nullptr && CachedCVar->GetInt() >= 0;
}

} // namespace ControlRigPhysicsEditor

#undef LOCTEXT_NAMESPACE
