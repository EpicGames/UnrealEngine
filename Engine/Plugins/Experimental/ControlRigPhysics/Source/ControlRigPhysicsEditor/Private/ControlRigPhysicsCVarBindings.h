// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"

class SWidget;

//======================================================================================================================
// Bindings tie a single CVar to a pair of Slate cells (Override / Value) hosted by a grid in the main widget.
//
// Lifecycle:
//   1. Main widget calls MakeShared<...>(cvar_name, ...).
//   2. Main widget calls Initialize(TWeakPtr<SWidget> Owner) after it can SharedThis itself.
//   3. Main widget calls BuildOverrideCell() / BuildValueCell() to populate grid slots.
//
// OnChangedDelegate is bound in Initialize so external console changes repaint the owning widget.
//
// Wrapped in a per-plugin namespace so the symbols don't collide with the (similarly-named) bindings
// in ControlRigDynamicsEditor when both editor modules are linked into the same target.
//======================================================================================================================

namespace ControlRigPhysicsEditor
{

//======================================================================================================================
// Simple 0/1 int32 CVar (e.g. ControlRig.Physics.EnableStepSolver). No override column (returns a spacer).
//======================================================================================================================
class FCVarSimpleToggleBinding : public TSharedFromThis<FCVarSimpleToggleBinding>
{
public:
	FCVarSimpleToggleBinding(const FString& InCVarName, const FText& InTooltip);
	~FCVarSimpleToggleBinding();

	void Initialize(TWeakPtr<SWidget> InOwnerWidget);

	TSharedRef<SWidget> BuildOverrideCell();
	TSharedRef<SWidget> BuildValueCell();

private:
	void HandleCVarChanged(IConsoleVariable* InVariable);
	ECheckBoxState GetValueCheckState() const;
	void OnValueCheckStateChanged(ECheckBoxState NewState);

	IConsoleVariable* CachedCVar = nullptr;
	FDelegateHandle OnChangedHandle;
	TWeakPtr<SWidget> OwnerWidget;
	FText Tooltip;
};

//======================================================================================================================
// Tri-state int32 override (-1 / 0 / 1). Override column shows the Apply checkbox.
//======================================================================================================================
class FCVarOverrideToggleBinding : public TSharedFromThis<FCVarOverrideToggleBinding>
{
public:
	FCVarOverrideToggleBinding(const FString& InCVarName, const FText& InTooltip);
	~FCVarOverrideToggleBinding();

	void Initialize(TWeakPtr<SWidget> InOwnerWidget);

	TSharedRef<SWidget> BuildOverrideCell();
	TSharedRef<SWidget> BuildValueCell();

private:
	void HandleCVarChanged(IConsoleVariable* InVariable);

	ECheckBoxState GetApplyCheckState() const;
	void OnApplyCheckStateChanged(ECheckBoxState NewState);

	ECheckBoxState GetValueCheckState() const;
	void OnValueCheckStateChanged(ECheckBoxState NewState);
	bool IsValueToggleEnabled() const;

	IConsoleVariable* CachedCVar = nullptr;
	FDelegateHandle OnChangedHandle;
	TWeakPtr<SWidget> OwnerWidget;
	FText Tooltip;
	int32 PendingValue = 1;
};

//======================================================================================================================
// Numeric override (float or int32). -1 (or any negative value) means "override inactive".
// Template implementation is inline - keeps SNumericEntryBox<T> out of the .cpp.
//======================================================================================================================
template <typename NumericType>
class FCVarOverrideNumericBinding : public TSharedFromThis<FCVarOverrideNumericBinding<NumericType>>
{
public:
	//==================================================================================================================
	FCVarOverrideNumericBinding(
		const FString& InCVarName,
		const FText& InTooltip,
		NumericType InDefaultValue,
		NumericType InMinAllowedValue)
		: Tooltip(InTooltip)
		, PendingValue(InDefaultValue)
		, MinAllowedValue(InMinAllowedValue)
	{
		CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);
		if (CachedCVar)
		{
			const NumericType CurrentValue = ReadCVar();
			if (CurrentValue >= NumericType(0))
			{
				PendingValue = CurrentValue;
			}
		}
	}

	//==================================================================================================================
	~FCVarOverrideNumericBinding()
	{
		if (CachedCVar && OnChangedHandle.IsValid())
		{
			CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
		}
	}

	//==================================================================================================================
	void Initialize(TWeakPtr<SWidget> InOwnerWidget)
	{
		OwnerWidget = InOwnerWidget;
		if (CachedCVar && !OnChangedHandle.IsValid())
		{
			OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
				this->AsShared(), &FCVarOverrideNumericBinding<NumericType>::HandleCVarChanged);
		}
	}

	//==================================================================================================================
	TSharedRef<SWidget> BuildOverrideCell()
	{
		return SNew(SCheckBox)
			.IsChecked(this, &FCVarOverrideNumericBinding<NumericType>::GetApplyCheckState)
			.OnCheckStateChanged(this, &FCVarOverrideNumericBinding<NumericType>::OnApplyCheckStateChanged)
			.IsEnabled(CachedCVar != nullptr)
			.ToolTipText(Tooltip);
	}

	//==================================================================================================================
	TSharedRef<SWidget> BuildValueCell()
	{
		return SNew(SNumericEntryBox<NumericType>)
			.Value(this, &FCVarOverrideNumericBinding<NumericType>::GetDisplayValue)
			.OnValueCommitted(this, &FCVarOverrideNumericBinding<NumericType>::OnValueCommitted)
			.IsEnabled(this, &FCVarOverrideNumericBinding<NumericType>::IsValueEntryEnabled)
			.MinDesiredValueWidth(80.0f)
			.AllowSpin(true)
			.MinValue(MinAllowedValue)
			.MinSliderValue(MinAllowedValue)
			.ToolTipText(Tooltip);
	}

private:
	//==================================================================================================================
	NumericType ReadCVar() const
	{
		if constexpr (std::is_same_v<NumericType, float>)
		{
			return CachedCVar->GetFloat();
		}
		else
		{
			return static_cast<NumericType>(CachedCVar->GetInt());
		}
	}

	//==================================================================================================================
	void WriteCVar(NumericType Value)
	{
		const FString ValueString = LexToString(Value);
		CachedCVar->Set(*ValueString, ECVF_SetByConsole);
	}

	//==================================================================================================================
	bool IsOverrideActive() const
	{
		return CachedCVar != nullptr && ReadCVar() >= NumericType(0);
	}

	//==================================================================================================================
	void HandleCVarChanged(IConsoleVariable* /*InVariable*/)
	{
		if (CachedCVar)
		{
			const NumericType CurrentValue = ReadCVar();
			if (CurrentValue >= NumericType(0))
			{
				PendingValue = CurrentValue;
			}
		}
		if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
		{
			Owner->Invalidate(EInvalidateWidgetReason::Paint);
		}
	}

	//==================================================================================================================
	ECheckBoxState GetApplyCheckState() const
	{
		return IsOverrideActive() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	//==================================================================================================================
	void OnApplyCheckStateChanged(ECheckBoxState NewState)
	{
		if (!CachedCVar)
		{
			return;
		}
		if (NewState == ECheckBoxState::Checked)
		{
			WriteCVar(PendingValue);
		}
		else
		{
			CachedCVar->Set(TEXT("-1"), ECVF_SetByConsole);
		}
	}

	//==================================================================================================================
	TOptional<NumericType> GetDisplayValue() const
	{
		if (IsOverrideActive())
		{
			return ReadCVar();
		}
		return PendingValue;
	}

	//==================================================================================================================
	void OnValueCommitted(NumericType NewValue, ETextCommit::Type /*CommitType*/)
	{
		if (NewValue < MinAllowedValue)
		{
			NewValue = MinAllowedValue;
		}
		PendingValue = NewValue;
		if (IsOverrideActive())
		{
			WriteCVar(PendingValue);
		}
	}

	//==================================================================================================================
	bool IsValueEntryEnabled() const
	{
		return IsOverrideActive();
	}

	IConsoleVariable* CachedCVar = nullptr;
	FDelegateHandle OnChangedHandle;
	TWeakPtr<SWidget> OwnerWidget;
	FText Tooltip;
	NumericType PendingValue;
	NumericType MinAllowedValue;
};

} // namespace ControlRigPhysicsEditor
