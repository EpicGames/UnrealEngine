// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigPhysicsEditorModule.h"
#include "SControlRigPhysicsDebugWidget.h"

#include "Containers/Ticker.h"
#include "Framework/Docking/SDockingTabStack.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/ScopeExit.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FControlRigPhysicsEditorModule"

namespace ControlRigPhysicsEditorModulePrivate
{
	static const FName DebugTabID(TEXT("ControlRigPhysicsDebug"));

	// The sibling debug tab from the ControlRigDynamics plugin. We don't depend on that module - we just
	// recognise its tab ID string. If that plugin isn't loaded, this lookup simply returns null.
	static const FName SiblingDebugTabID(TEXT("ControlRigDynamicsDebug"));

	// Tracks the currently-live debug tab so re-invocations bring it to front instead of spawning a duplicate.
	// FindExistingLiveTab uses strict FTabId equality (TabType + InstanceId), and InsertNewDocumentTab assigns
	// our relocated copy a non-INDEX_NONE InstanceId, so a TabType-only lookup via that API doesn't find it.
	// A weak pointer here sidesteps the issue: we update it both on initial spawn and after relocation.
	static TWeakPtr<SDockTab> CurrentDebugTab;

	// Set in ShutdownModule, cleared in StartupModule (so module reload starts fresh). The deferred
	// relocation ticker may still be called after shutdown begins; when it does, this flag tells it
	// to early-return before touching FGlobalTabmanager (which may be mid-teardown) or scheduling
	// further Slate work.
	static bool bIsShuttingDown = false;

	// Handles for any in-flight relocation tickers, so ShutdownModule can cancel them rather than
	// rely solely on the bIsShuttingDown flag check inside the lambda. Each scheduled lambda removes
	// its own handle from this array when it fires (via ON_SCOPE_EXIT), so the array's steady-state
	// size is the number of currently-pending tickers - typically zero or one.
	static TArray<FTSTicker::FDelegateHandle> PendingTickerHandles;

	//==================================================================================================================
	// After our debug tab spawns in its default location, check (next tick) whether the sibling debug tab is alive
	// in a different stack. If so, close the original and re-insert a fresh copy next to the sibling via
	// InsertNewDocumentTab - that's the only public path that sets the layout identifier (SetLayoutIdentifier is
	// protected) AND inserts into the target stack in one call.
	//==================================================================================================================
	static void ScheduleSiblingDockingIfNeeded(const TSharedRef<SDockTab>& NewlySpawnedTab, const FText& TabLabel)
	{
		TWeakPtr<SDockTab> WeakNewTab = NewlySpawnedTab;

		// Heap-allocated handle holder so the lambda can clean itself out of PendingTickerHandles when
		// it fires. We assign *HandleHolder after AddTicker returns; the lambda only reads it on tick,
		// by which point the assignment has happened.
		TSharedRef<FTSTicker::FDelegateHandle> HandleHolder = MakeShared<FTSTicker::FDelegateHandle>();

		*HandleHolder = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
			[WeakNewTab, TabLabel, HandleHolder](float /*DeltaTime*/) -> bool
			{
				ON_SCOPE_EXIT
				{
					PendingTickerHandles.RemoveSingleSwap(*HandleHolder);
				};

				if (bIsShuttingDown)
				{
					return false;
				}

				TSharedPtr<SDockTab> NewTab = WeakNewTab.Pin();
				if (!NewTab.IsValid())
				{
					return false;
				}

				const TSharedPtr<SDockTab> Sibling =
					FGlobalTabmanager::Get()->FindExistingLiveTab(FTabId(SiblingDebugTabID));
				if (!Sibling.IsValid())
				{
					return false;
				}

				const TSharedPtr<SDockingTabStack> SiblingStack = Sibling->GetParentDockTabStack();
				const TSharedPtr<SDockingTabStack> CurrentStack = NewTab->GetParentDockTabStack();
				if (!SiblingStack.IsValid() || !CurrentStack.IsValid() || SiblingStack == CurrentStack)
				{
					return false;
				}

				const TSharedRef<SDockTab> ReplacementTab = SNew(SDockTab)
					.TabRole(ETabRole::NomadTab)
					.Label(TabLabel)
					[
						SNew(SControlRigPhysicsDebugWidget)
					];

				// Close original first so the tab manager doesn't briefly track two tabs with the same TabType.
				NewTab->RequestCloseTab();

				FGlobalTabmanager::Get()->InsertNewDocumentTab(
					SiblingDebugTabID,
					DebugTabID,
					FTabManager::FLiveTabSearch(),
					ReplacementTab);

				CurrentDebugTab = ReplacementTab;

				return false;
			}), 0.0f);
		PendingTickerHandles.Add(*HandleHolder);
	}
}

//======================================================================================================================
void FControlRigPhysicsEditorModule::StartupModule()
{
	using namespace ControlRigPhysicsEditorModulePrivate;

	// Reset shutdown flag in case the module is being reloaded (the static persists across reload).
	bIsShuttingDown = false;

	const FSlateIcon TabIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings");

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			DebugTabID,
			FOnSpawnTab::CreateRaw(this, &FControlRigPhysicsEditorModule::OnSpawnDebugTab))
		.SetDisplayName(LOCTEXT("DebugTabTitle", "ControlRig Physics Debug"))
		.SetTooltipText(LOCTEXT("DebugTabTooltip",
			"Opens the ControlRig Physics debug panel: toggle visualization, override solver time-step settings, etc."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetIcon(TabIcon)
		// The default reuse method (Spawner->SpawnedTabPtr.Pin()) goes stale after we close-and-reopen during
		// relocation, and FindExistingLiveTab uses strict FTabId equality so it can't find the relocated tab
		// either (InstanceId mismatch). Track the current tab ourselves; updated on spawn and after relocation.
		.SetReuseTabMethod(FOnFindTabToReuse::CreateLambda([](const FTabId& /*InTabId*/)
		{
			return ControlRigPhysicsEditorModulePrivate::CurrentDebugTab.Pin();
		}));

	// Extend the Control Rig asset editor's toolbar with a button that opens the debug tab.
	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(
		this, &FControlRigPhysicsEditorModule::RegisterMenus));
}

//======================================================================================================================
void FControlRigPhysicsEditorModule::ShutdownModule()
{
	using namespace ControlRigPhysicsEditorModulePrivate;

	bIsShuttingDown = true;

	// Cancel any in-flight relocation tickers. RemoveTicker is a no-op if the handle has already fired.
	for (const FTSTicker::FDelegateHandle& Handle : PendingTickerHandles)
	{
		FTSTicker::GetCoreTicker().RemoveTicker(Handle);
	}
	PendingTickerHandles.Reset();

	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(DebugTabID);
}

//======================================================================================================================
void FControlRigPhysicsEditorModule::RegisterMenus()
{
	using namespace ControlRigPhysicsEditorModulePrivate;

	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* ControlRigToolBar = UToolMenus::Get()->ExtendMenu("AssetEditor.ControlRigEditor.ToolBar");
	if (!ControlRigToolBar)
	{
		return;
	}

	FToolMenuSection& Section = ControlRigToolBar->FindOrAddSection("ControlRigPhysicsDebug");

	Section.AddEntry(FToolMenuEntry::InitToolBarButton(
		"OpenControlRigPhysicsDebug",
		FUIAction(FExecuteAction::CreateLambda([TabID = DebugTabID]()
		{
			FGlobalTabmanager::Get()->TryInvokeTab(FTabId(TabID));
		})),
		LOCTEXT("ToolBarButtonLabel", "Physics Debug"),
		LOCTEXT("ToolBarButtonTooltip", "Open the ControlRig Physics debug panel."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Settings")));
}

//======================================================================================================================
TSharedRef<SDockTab> FControlRigPhysicsEditorModule::OnSpawnDebugTab(const FSpawnTabArgs& /*SpawnTabArgs*/)
{
	const FText TabLabel = LOCTEXT("DebugTabTitle", "ControlRig Physics Debug");

	const TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.Label(TabLabel)
		[
			SNew(SControlRigPhysicsDebugWidget)
		];

	ControlRigPhysicsEditorModulePrivate::CurrentDebugTab = NewTab;
	ControlRigPhysicsEditorModulePrivate::ScheduleSiblingDockingIfNeeded(NewTab, TabLabel);

	return NewTab;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FControlRigPhysicsEditorModule, ControlRigPhysicsEditor)
