// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigPhysicsDebugWidget.h"

#include "ControlRigPhysicsCVarBindings.h"

#include "HAL/IConsoleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SControlRigPhysicsDebugWidget"

using namespace ControlRigPhysicsEditor;

namespace ControlRigPhysicsDebugWidgetPrivate
{
	// Kept in a single place so the layout and the Clear Overrides action agree on the list. Every CVar here
	// uses -1 as its "override inactive" sentinel; the simple-toggle CVars are reset separately.
	static const TCHAR* const AllOverrideCVarNames[] =
	{
		TEXT("ControlRig.Physics.FixedTimeStepOverride"),
		TEXT("ControlRig.Physics.MaxTimeStepsOverride"),
		TEXT("ControlRig.Physics.MaxDeltaTimeOverride"),
		TEXT("ControlRig.Physics.ShowBodiesOverride"),
		TEXT("ControlRig.Physics.ShowCentreOfMassOverride"),
		TEXT("ControlRig.Physics.ShowJointsOverride"),
		TEXT("ControlRig.Physics.ShowControlsOverride"),
		TEXT("ControlRig.Physics.ShowActiveContactsOverride"),
		TEXT("ControlRig.Physics.ShowInactiveContactsOverride"),
		TEXT("ControlRig.Physics.ShowWorldObjectsOverride"),
		TEXT("ControlRig.Physics.ShowWorldOverlapBoxOverride"),
	};

	constexpr int32 ColOverride = 0;
	constexpr int32 ColName     = 1;
	constexpr int32 ColValue    = 2;
	constexpr int32 NumColumns  = 3;

	//==================================================================================================================
	static TSharedRef<SWidget> MakeSectionHeader(const FText& InText)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(6.0f, 4.0f))
			[
				SNew(STextBlock)
				.Text(InText)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			];
	}

	//==================================================================================================================
	static TSharedRef<SWidget> MakeLabelCell(const FText& InText, const FText& InTooltip)
	{
		return SNew(STextBlock)
			.Text(InText)
			.ToolTipText(InTooltip);
	}
}

//======================================================================================================================
void SControlRigPhysicsDebugWidget::Construct(const FArguments& /*InArgs*/)
{
	using namespace ControlRigPhysicsDebugWidgetPrivate;

	// --- Tooltips ---------------------------------------------------------------------------------------------

	const FText EnableStepSolverTooltip = LOCTEXT("EnableStepSolverTooltip",
		"When disabled, every Step Physics Solver call is skipped. Useful for pausing the simulation while debugging (though visualization will be disabled too).");
	const FText FixedTimeStepTooltip = LOCTEXT("FixedTimeStepOverrideTooltip",
		"When applied, overrides the solver's fixed time step (seconds). 0 forces a variable timestep; a positive value forces a fixed timestep.");
	const FText MaxTimeStepsTooltip = LOCTEXT("MaxTimeStepsOverrideTooltip",
		"When applied, overrides the maximum number of substeps the solver may run per frame.");
	const FText MaxDeltaTimeTooltip = LOCTEXT("MaxDeltaTimeOverrideTooltip",
		"When applied, overrides the maximum delta time (seconds) the solver will integrate in a single frame. Set to zero to effectively pause the simulation.");

	const FText AllowVisualizationTooltip = LOCTEXT("AllowVisualizationTooltip",
		"Master allow-switch for physics debug visualization. When off, all debug drawing is suppressed regardless of the rig's settings (useful for profiling). When on, drawing is gated by the rig's individual show flags.");
	const FText ShowBodiesTooltip = LOCTEXT("ShowBodiesTooltip",
		"Overrides whether physics body collision shapes are drawn.");
	const FText ShowCentreOfMassTooltip = LOCTEXT("ShowCentreOfMassTooltip",
		"Overrides whether per-body centre-of-mass markers are drawn.");
	const FText ShowJointsTooltip = LOCTEXT("ShowJointsTooltip",
		"Overrides whether joint constraints between bodies are drawn.");
	const FText ShowControlsTooltip = LOCTEXT("ShowControlsTooltip",
		"Overrides whether physics controls are drawn.");
	const FText ShowActiveContactsTooltip = LOCTEXT("ShowActiveContactsTooltip",
		"Overrides whether active contact points are drawn (red).");
	const FText ShowInactiveContactsTooltip = LOCTEXT("ShowInactiveContactsTooltip",
		"Overrides whether inactive contact points are drawn (silver).");
	const FText ShowWorldObjectsTooltip = LOCTEXT("ShowWorldObjectsTooltip",
		"Overrides whether kinematic mirrors of world primitives queried for collision are drawn.");
	const FText ShowWorldOverlapBoxTooltip = LOCTEXT("ShowWorldOverlapBoxTooltip",
		"Overrides whether the bounding box used to query world primitives is drawn.");
	const FText ShowSimulationSpaceInfoTooltip = LOCTEXT("ShowSimulationSpaceInfoTooltip",
		"Shows on-screen text with simulation timing and the simulation-space transform/velocity.");

	const FText EnableDrawInterfaceInGameTooltip = LOCTEXT("EnableDrawInterfaceInGameTooltip",
		"Enables/disables general Control Rig debug drawing during PIE and in-game. "
		"Required to be on for any of the Visualization toggles above to take effect at runtime.");
	const FText AnimNodeControlRigDebugTooltip = LOCTEXT("AnimNodeControlRigDebugTooltip",
		"Enables/disables debug drawing for AnimNode_ControlRigBase. Required for the Visualization toggles above to take effect "
		"when the rig is driven by an AnimGraph node. Only registered in builds with ENABLE_ANIM_DEBUG.");

	// --- Create bindings --------------------------------------------------------------------------------------

	EnableStepSolverBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.Physics.EnableStepSolver"), EnableStepSolverTooltip);
	FixedTimeStepBinding = MakeShared<FCVarOverrideNumericBinding<float>>(
		TEXT("ControlRig.Physics.FixedTimeStepOverride"), FixedTimeStepTooltip, 1.0f / 60.0f, 0.0f);
	MaxTimeStepsBinding = MakeShared<FCVarOverrideNumericBinding<int32>>(
		TEXT("ControlRig.Physics.MaxTimeStepsOverride"), MaxTimeStepsTooltip, 4, 0);
	MaxDeltaTimeBinding = MakeShared<FCVarOverrideNumericBinding<float>>(
		TEXT("ControlRig.Physics.MaxDeltaTimeOverride"), MaxDeltaTimeTooltip, 1.0f / 30.0f, 0.0f);

	AllowVisualizationBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.Physics.AllowVisualization"), AllowVisualizationTooltip);
	ShowBodiesBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowBodiesOverride"), ShowBodiesTooltip);
	ShowCentreOfMassBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowCentreOfMassOverride"), ShowCentreOfMassTooltip);
	ShowJointsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowJointsOverride"), ShowJointsTooltip);
	ShowControlsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowControlsOverride"), ShowControlsTooltip);
	ShowActiveContactsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowActiveContactsOverride"), ShowActiveContactsTooltip);
	ShowInactiveContactsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowInactiveContactsOverride"), ShowInactiveContactsTooltip);
	ShowWorldObjectsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowWorldObjectsOverride"), ShowWorldObjectsTooltip);
	ShowWorldOverlapBoxBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Physics.ShowWorldOverlapBoxOverride"), ShowWorldOverlapBoxTooltip);
	ShowSimulationSpaceInfoBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.Physics.ShowSimulationSpaceInfo"), ShowSimulationSpaceInfoTooltip);

	EnableDrawInterfaceInGameBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.EnableDrawInterfaceInGame"), EnableDrawInterfaceInGameTooltip);
	AnimNodeControlRigDebugBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("a.AnimNode.ControlRig.Debug"), AnimNodeControlRigDebugTooltip);

	// --- Build grid -------------------------------------------------------------------------------------------

	TSharedRef<SGridPanel> Grid = SNew(SGridPanel).FillColumn(ColName, 1.0f);

	const FMargin CellPadding(4.0f, 3.0f);
	int32 Row = 0;

	auto AddSectionHeader = [&Grid, &Row](const FText& InText)
	{
		Grid->AddSlot(ColOverride, Row)
			.ColumnSpan(NumColumns)
			.Padding(FMargin(0.0f, 6.0f, 0.0f, 2.0f))
			[
				MakeSectionHeader(InText)
			];
		++Row;
	};

	auto AddBindingRow = [&Grid, &Row, CellPadding](
		const TSharedRef<SWidget>& OverrideCell,
		const FText& Label,
		const FText& Tooltip,
		const TSharedRef<SWidget>& ValueCell)
	{
		Grid->AddSlot(ColOverride, Row)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				OverrideCell
			];
		Grid->AddSlot(ColName, Row)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				MakeLabelCell(Label, Tooltip)
			];
		Grid->AddSlot(ColValue, Row)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				ValueCell
			];
		++Row;
	};

	// --- Simulation -------------------------------------------------------------------------------------------

	AddSectionHeader(LOCTEXT("SimulationSection", "Simulation"));

	AddBindingRow(
		EnableStepSolverBinding->BuildOverrideCell(),
		LOCTEXT("EnableStepSolverLabel", "Enable Step Solver"), EnableStepSolverTooltip,
		EnableStepSolverBinding->BuildValueCell());

	AddBindingRow(
		FixedTimeStepBinding->BuildOverrideCell(),
		LOCTEXT("FixedTimeStepOverrideLabel", "Fixed Time Step"), FixedTimeStepTooltip,
		FixedTimeStepBinding->BuildValueCell());

	AddBindingRow(
		MaxTimeStepsBinding->BuildOverrideCell(),
		LOCTEXT("MaxTimeStepsOverrideLabel", "Max Time Steps"), MaxTimeStepsTooltip,
		MaxTimeStepsBinding->BuildValueCell());

	AddBindingRow(
		MaxDeltaTimeBinding->BuildOverrideCell(),
		LOCTEXT("MaxDeltaTimeOverrideLabel", "Max Delta Time"), MaxDeltaTimeTooltip,
		MaxDeltaTimeBinding->BuildValueCell());

	// --- Visualization ----------------------------------------------------------------------------------------

	AddSectionHeader(LOCTEXT("VisualizationSection", "Visualization"));

	AddBindingRow(
		AllowVisualizationBinding->BuildOverrideCell(),
		LOCTEXT("AllowVisualizationLabel", "Allow Visualization"), AllowVisualizationTooltip,
		AllowVisualizationBinding->BuildValueCell());

	AddBindingRow(
		ShowBodiesBinding->BuildOverrideCell(),
		LOCTEXT("ShowBodiesLabel", "Show Bodies"), ShowBodiesTooltip,
		ShowBodiesBinding->BuildValueCell());

	AddBindingRow(
		ShowCentreOfMassBinding->BuildOverrideCell(),
		LOCTEXT("ShowCentreOfMassLabel", "Show Centre of Mass"), ShowCentreOfMassTooltip,
		ShowCentreOfMassBinding->BuildValueCell());

	AddBindingRow(
		ShowJointsBinding->BuildOverrideCell(),
		LOCTEXT("ShowJointsLabel", "Show Joints"), ShowJointsTooltip,
		ShowJointsBinding->BuildValueCell());

	AddBindingRow(
		ShowControlsBinding->BuildOverrideCell(),
		LOCTEXT("ShowControlsLabel", "Show Controls"), ShowControlsTooltip,
		ShowControlsBinding->BuildValueCell());

	AddBindingRow(
		ShowActiveContactsBinding->BuildOverrideCell(),
		LOCTEXT("ShowActiveContactsLabel", "Show Active Contacts"), ShowActiveContactsTooltip,
		ShowActiveContactsBinding->BuildValueCell());

	AddBindingRow(
		ShowInactiveContactsBinding->BuildOverrideCell(),
		LOCTEXT("ShowInactiveContactsLabel", "Show Inactive Contacts"), ShowInactiveContactsTooltip,
		ShowInactiveContactsBinding->BuildValueCell());

	AddBindingRow(
		ShowWorldObjectsBinding->BuildOverrideCell(),
		LOCTEXT("ShowWorldObjectsLabel", "Show World Objects"), ShowWorldObjectsTooltip,
		ShowWorldObjectsBinding->BuildValueCell());

	AddBindingRow(
		ShowWorldOverlapBoxBinding->BuildOverrideCell(),
		LOCTEXT("ShowWorldOverlapBoxLabel", "Show World Overlap Box"), ShowWorldOverlapBoxTooltip,
		ShowWorldOverlapBoxBinding->BuildValueCell());

	// --- In-Game --------------------------------------------------------------------------------------------
	// Engine-wide gates that suppress all Control Rig debug drawing in PIE / shipped game. These are independent
	// of the rig's Visualization flags above; both have to be on for any of the rows above to take effect at runtime.

	AddSectionHeader(LOCTEXT("InGameSection", "In-Game"));

	AddBindingRow(
		EnableDrawInterfaceInGameBinding->BuildOverrideCell(),
		LOCTEXT("EnableDrawInterfaceInGameLabel", "ControlRig.EnableDrawInterfaceInGame"), EnableDrawInterfaceInGameTooltip,
		EnableDrawInterfaceInGameBinding->BuildValueCell());

	AddBindingRow(
		AnimNodeControlRigDebugBinding->BuildOverrideCell(),
		LOCTEXT("AnimNodeControlRigDebugLabel", "a.AnimNode.ControlRig.Debug"), AnimNodeControlRigDebugTooltip,
		AnimNodeControlRigDebugBinding->BuildValueCell());

	// --- Debug Info -----------------------------------------------------------------------------------------
	// Distinct from the Visualization section: these toggles are not gated by Allow Visualization,
	// since they don't go through the per-rig Draw() path.

	AddSectionHeader(LOCTEXT("DebugInfoSection", "Debug Info"));

	AddBindingRow(
		ShowSimulationSpaceInfoBinding->BuildOverrideCell(),
		LOCTEXT("ShowSimulationSpaceInfoLabel", "Show Simulation Space Info"), ShowSimulationSpaceInfoTooltip,
		ShowSimulationSpaceInfoBinding->BuildValueCell());

	// --- Assemble top-level layout ---------------------------------------------------------------------------

	ChildSlot
	.Padding(4.0f)
	[
		SNew(SBox)
		.MinDesiredWidth(460.0f)
		[
			SNew(SVerticalBox)

			// Scrollable area: the grid. Takes all remaining vertical space and scrolls when squished.
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					Grid
				]
			]

			// Separator and Clear Overrides button stay pinned at the bottom, outside the scroll area.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f)
			[
				SNew(SSeparator)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(4.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearOverridesButton", "Clear Overrides"))
				.ToolTipText(LOCTEXT("ClearOverridesTooltip",
					"Clears the override checkboxes, and resets only the values that don't have an override checkbox to their defaults. "
					"Values guarded by an override checkbox are preserved, so re-enabling the checkbox restores what you had."))
				.OnClicked(this, &SControlRigPhysicsDebugWidget::OnClearOverridesClicked)
			]
		]
	];

	// --- Hook OnChangedDelegate listeners now that SharedThis is valid ---------------------------------------

	const TWeakPtr<SWidget> WeakSelf = SharedThis(this);

	EnableStepSolverBinding->Initialize(WeakSelf);
	FixedTimeStepBinding->Initialize(WeakSelf);
	MaxTimeStepsBinding->Initialize(WeakSelf);
	MaxDeltaTimeBinding->Initialize(WeakSelf);
	AllowVisualizationBinding->Initialize(WeakSelf);
	ShowBodiesBinding->Initialize(WeakSelf);
	ShowCentreOfMassBinding->Initialize(WeakSelf);
	ShowJointsBinding->Initialize(WeakSelf);
	ShowControlsBinding->Initialize(WeakSelf);
	ShowActiveContactsBinding->Initialize(WeakSelf);
	ShowInactiveContactsBinding->Initialize(WeakSelf);
	ShowWorldObjectsBinding->Initialize(WeakSelf);
	ShowWorldOverlapBoxBinding->Initialize(WeakSelf);
	ShowSimulationSpaceInfoBinding->Initialize(WeakSelf);
	EnableDrawInterfaceInGameBinding->Initialize(WeakSelf);
	AnimNodeControlRigDebugBinding->Initialize(WeakSelf);
}

//======================================================================================================================
FReply SControlRigPhysicsDebugWidget::OnClearOverridesClicked()
{
	using namespace ControlRigPhysicsDebugWidgetPrivate;

	IConsoleManager& ConsoleManager = IConsoleManager::Get();

	for (const TCHAR* CVarName : AllOverrideCVarNames)
	{
		if (IConsoleVariable* CVar = ConsoleManager.FindConsoleVariable(CVarName))
		{
			CVar->Set(TEXT("-1"), ECVF_SetByConsole);
		}
	}

	if (IConsoleVariable* EnableStepSolver = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.Physics.EnableStepSolver")))
	{
		EnableStepSolver->Set(TEXT("1"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* AllowVisualization = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.Physics.AllowVisualization")))
	{
		AllowVisualization->Set(TEXT("1"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* ShowSimSpace = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.Physics.ShowSimulationSpaceInfo")))
	{
		ShowSimSpace->Set(TEXT("0"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* EnableDrawInGame = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.EnableDrawInterfaceInGame")))
	{
		EnableDrawInGame->Set(TEXT("0"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* AnimNodeDebug = ConsoleManager.FindConsoleVariable(TEXT("a.AnimNode.ControlRig.Debug")))
	{
		AnimNodeDebug->Set(TEXT("0"), ECVF_SetByConsole);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
