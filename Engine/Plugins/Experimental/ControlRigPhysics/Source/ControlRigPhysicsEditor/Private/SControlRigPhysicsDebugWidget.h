// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigPhysicsCVarBindings.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

//======================================================================================================================
// Top-level debug widget for ControlRigPhysics. Lays out the CVar controls in a 3-column grid
// (Override | Name | Value) so the Apply checkboxes and value widgets align neatly across rows.
//======================================================================================================================
class SControlRigPhysicsDebugWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlRigPhysicsDebugWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FReply OnClearOverridesClicked();

	// Binding lifetime matches the widget. Kept as members so OnChangedDelegate bindings stay alive.
	// The binding types are namespaced (ControlRigPhysicsEditor::) to avoid duplicate-symbol clashes
	// with the parallel set declared in ControlRigDynamicsEditor.
	TSharedPtr<ControlRigPhysicsEditor::FCVarSimpleToggleBinding>           EnableStepSolverBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideNumericBinding<float>> FixedTimeStepBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideNumericBinding<int32>> MaxTimeStepsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideNumericBinding<float>> MaxDeltaTimeBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarSimpleToggleBinding>           AllowVisualizationBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowBodiesBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowCentreOfMassBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowJointsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowControlsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowActiveContactsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowInactiveContactsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowWorldObjectsBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarOverrideToggleBinding>         ShowWorldOverlapBoxBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarSimpleToggleBinding>           EnableDrawInterfaceInGameBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarSimpleToggleBinding>           AnimNodeControlRigDebugBinding;
	TSharedPtr<ControlRigPhysicsEditor::FCVarSimpleToggleBinding>           ShowSimulationSpaceInfoBinding;
};
